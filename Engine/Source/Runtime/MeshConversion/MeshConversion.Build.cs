// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MeshConversion : ModuleRules
{	
	public MeshConversion(ReadOnlyTargetRules Target) : base(Target)
	{
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		// Note: The module purposefully doesn't have a dependency on CoreUObject.
		// Avoiding CoreUObject and Engine for this MeshConversion functionality
		// keeps the door open for writing standalone command-line programs and unit tests
		// (which won't have UObject garbage collection).
		// For conversions that depend on Engine types, see the MeshConversionEngineTypes module
        PublicDependencyModuleNames.AddRange(
			new string[] {
                "Core",
                "MeshDescription",
				"StaticMeshDescription",
				"SkeletalMeshDescription",
				"GeometryCore"
            }
		);
    }
}
