// Copyright Epic Games, Inc. All Rights Reserved. 

#include "DynamicMeshToMeshDescription.h"
#include "SkeletalMeshAttributes.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMeshOverlay.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "DynamicMesh/DynamicBoneAttribute.h"
#include "DynamicMesh/DynamicVertexAttribute.h"
#include "MeshDescriptionBuilder.h"
#include "DynamicMesh/MeshTangents.h"
#include "DynamicMesh/NonManifoldMappingSupport.h"
#include "Util/ColorConstants.h"
#include "Async/ParallelFor.h"

using namespace UE::Geometry;

namespace DynamicMeshToMeshDescriptionConversionHelper
{
	// NOTE: assumes the order of triangles in the MeshIn correspond to the ordering over tris on MeshOut
	// This matches conversion currently used in MeshDescriptionToDynamicMesh.cpp, but if that changes we will need to change this function to match!
	template <typename OutAttributeType, int VecLen, typename InAttributeType>
	void SetAttributesFromOverlay(
		const FDynamicMesh3* MeshInArg, const FMeshDescription& MeshOutArg,
		TVertexInstanceAttributesRef<OutAttributeType>& InstanceAttrib, const TDynamicMeshVectorOverlay<float, VecLen, InAttributeType>* Overlay, OutAttributeType DefaultValue, int AttribIndex=0)
	{
		if (!ensureMsgf(AttribIndex >= 0 && AttribIndex < InstanceAttrib.GetNumChannels(), TEXT("Failed to set attributes from overlay; invalid AttribIndex %d"), AttribIndex))
		{
			return;
		}
		for (const FTriangleID TriangleID : MeshOutArg.Triangles().GetElementIDs())
		{
			TArrayView<const FVertexInstanceID> InstanceTri = MeshOutArg.GetTriangleVertexInstances(TriangleID);
			auto ValidTargetElement = [NumAttribElements = InstanceAttrib.GetNumElements()](const FVertexInstanceID ID) -> bool 
			{
				int32 IDValue = ID.GetValue();
				return ensureMsgf(IDValue >= 0 && IDValue < NumAttribElements, TEXT("Failed to set attributes from overlay; invalid target vertex instance %d in mesh with %d instances"), IDValue, NumAttribElements);
			};
			if (!ValidTargetElement(InstanceTri[0]) || !ValidTargetElement(InstanceTri[1]) || !ValidTargetElement(InstanceTri[2]))
			{
				continue;
			}
			
			auto SetDefaults = [&InstanceAttrib, &InstanceTri, AttribIndex, &DefaultValue]()
			{
				InstanceAttrib.Set(InstanceTri[0], AttribIndex, DefaultValue);
				InstanceAttrib.Set(InstanceTri[1], AttribIndex, DefaultValue);
				InstanceAttrib.Set(InstanceTri[2], AttribIndex, DefaultValue);
			};

			int32 MeshInTriIdx = TriangleID.GetValue();
			
			if (Overlay->IsSetTriangle(MeshInTriIdx) && 
				ensureMsgf(MeshInArg->IsTriangle(MeshInTriIdx), TEXT("Failed to set attributes from overlay; invalid TriIdx %d not found in mesh with %d Tri IDs and %d Tri Count"), MeshInTriIdx, MeshInArg->MaxTriangleID(), MeshInArg->TriangleCount()))
			{
				FIndex3i OverlayEls = Overlay->GetTriangle(MeshInTriIdx);
				if (!ensureMsgf(Overlay->IsElement(OverlayEls.A) && Overlay->IsElement(OverlayEls.B) && Overlay->IsElement(OverlayEls.C), TEXT("Failed to set attributes from overlay; invalid overlay triangle w/ indices (%d, %d, %d) on overlay with %d element IDs"), OverlayEls.A, OverlayEls.B, OverlayEls.C, Overlay->MaxElementID()))
				{
					SetDefaults();
				}
				else
				{
					InstanceAttrib.Set(InstanceTri[0], AttribIndex, OutAttributeType(Overlay->GetElement(OverlayEls.A)));
					InstanceAttrib.Set(InstanceTri[1], AttribIndex, OutAttributeType(Overlay->GetElement(OverlayEls.B)));
					InstanceAttrib.Set(InstanceTri[2], AttribIndex, OutAttributeType(Overlay->GetElement(OverlayEls.C)));
				}
			}
			else
			{
				SetDefaults();
			}
		}
	}

	void ConvertVertices(
		const FDynamicMesh3& InMesh,
		FMeshDescriptionBuilder& InBuilder,
		const bool bReconstructNonManifoldMesh,
		TArray<FVertexID>& OutVertexMappings
		)
	{
		OutVertexMappings.SetNum(InMesh.MaxVertexID());
		
		FNonManifoldMappingSupport ManifoldMapping(InMesh);
		if (bReconstructNonManifoldMesh && ManifoldMapping.IsNonManifoldVertexInSource())
		{
			// Figure out the actual vertex count we need. For non-mapping vertices, we count them separately and
			// then tally them all up. The new vertices will be appended after the original vertices.
			int32 MaxOriginalVertexID = 0;
			int32 NumNewVertices = 0;
			for (const int VertID : InMesh.VertexIndicesItr())
			{
				int32 OriginalVertID = ManifoldMapping.GetOriginalNonManifoldVertexID(VertID);
				if (OriginalVertID != INDEX_NONE)
				{
					MaxOriginalVertexID = FMath::Max(MaxOriginalVertexID, OriginalVertID);
				}
				else
				{
					NumNewVertices++;
				}
			}
		
			int32 NewVertexID = MaxOriginalVertexID + 1;

			InBuilder.ReserveNewVertices(NewVertexID + NumNewVertices);
			TSet<int32> DefinedVertexIDs;

			for (int VertID : InMesh.VertexIndicesItr())
			{
				const FVector3d Position = InMesh.GetVertex(VertID);
				int32 OriginalVertID = ManifoldMapping.GetOriginalNonManifoldVertexID(VertID);
				if (OriginalVertID != INDEX_NONE)
				{
					// FMeshDescription::CreateVertexWithID is not happy if we try to create the same vertex twice.
					if (!DefinedVertexIDs.Contains(OriginalVertID))
					{
						OutVertexMappings[VertID] = InBuilder.AppendVertexWithId(OriginalVertID, Position);
						DefinedVertexIDs.Add(OriginalVertID);
					}
					else
					{
						OutVertexMappings[VertID] = FVertexID{OriginalVertID};
					}
				}
				else
				{
					OutVertexMappings[VertID] = InBuilder.AppendVertexWithId(NewVertexID++, Position);
				}
			}
		}
		else
		{
			// allocate
			InBuilder.ReserveNewVertices(InMesh.VertexCount());
		
			// create "vertex buffer" in MeshDescription
			for (int VertID : InMesh.VertexIndicesItr())
			{
				OutVertexMappings[VertID] = InBuilder.AppendVertex(InMesh.GetVertex(VertID));
			}
		}
	}

	void ValidateMaterialID(int32& MaterialID, int32 MaxValidMaterialID)
	{
		if (MaxValidMaterialID > INDEX_NONE && (MaterialID < 0 || MaterialID > MaxValidMaterialID))
		{
			MaterialID = 0;
		}
	}
}


void FDynamicMeshToMeshDescription::Update(const FDynamicMesh3* MeshIn, FMeshDescription& MeshOut, bool bUpdateNormals, bool bUpdateTangents, bool bUpdateUVs)
{
	FMeshDescriptionBuilder Builder;
	Builder.SetMeshDescription(&MeshOut);

	check(MeshIn->IsCompactV());

	// can't currently update the shared UV connectivity data structure on the MeshDescription :(
	//  -- see FDynamicMeshToMeshDescription::UpdateAttributes()
	// after this has been fixed, please update USimpleDynamicMeshComponent::Bake() to use the Update() path accordingly  
	check(bUpdateUVs == false);

	// update positions
	int32 NumVertices = MeshOut.Vertices().Num();
	check(NumVertices <= MeshIn->VertexCount());
	for (int32 VertID = 0; VertID < NumVertices; ++VertID)
	{
		Builder.SetPosition(FVertexID(VertID), (FVector)MeshIn->GetVertex(VertID));
	}

	UpdateAttributes(MeshIn, MeshOut, bUpdateNormals, bUpdateTangents, bUpdateUVs);
}



void FDynamicMeshToMeshDescription::UpdateAttributes(const FDynamicMesh3* MeshIn, FMeshDescription& MeshOut, bool bUpdateNormals, bool bUpdateTangents, bool bUpdateUVs)
{
	check(MeshIn->IsCompactV());


	FStaticMeshAttributes Attributes(MeshOut);


	if (bUpdateNormals)
	{
		TVertexInstanceAttributesRef<FVector3f> InstanceAttrib = Attributes.GetVertexInstanceNormals();
		bool bIsValidDst = InstanceAttrib.IsValid();
		ensureMsgf(bIsValidDst, TEXT("Trying to update normals on a MeshDescription that has no normal attributes"));
		if (bIsValidDst)
		{
			const FDynamicMeshNormalOverlay* Overlay = MeshIn->HasAttributes() ? MeshIn->Attributes()->PrimaryNormals() : nullptr;
			if (Overlay)
			{
				check(MeshIn->TriangleCount() == MeshOut.Triangles().Num())
				DynamicMeshToMeshDescriptionConversionHelper::SetAttributesFromOverlay(MeshIn, MeshOut, InstanceAttrib, Overlay, FVector3f::ZeroVector);
			}
			else
			{
				check(MeshIn->VertexCount() == MeshOut.Vertices().Num());
				for (int VertID : MeshIn->VertexIndicesItr())
				{
					FVector3f Normal = MeshIn->GetVertexNormal(VertID);
					for (FVertexInstanceID InstanceID : MeshOut.GetVertexVertexInstanceIDs(FVertexID(VertID)))
					{
						InstanceAttrib.Set(InstanceID, Normal);
					}
				}
			}
		}
	}

	if (bUpdateTangents)
	{
		UpdateTangents(MeshIn, MeshOut);
	}
	

	if (bUpdateUVs)
	{
		TVertexInstanceAttributesRef<FVector2f> InstanceAttrib = Attributes.GetVertexInstanceUVs();
		ensureMsgf(InstanceAttrib.IsValid(), TEXT("Trying to update UVs on a MeshDescription that has no texture coordinate attributes"));
		if (InstanceAttrib.IsValid())
		{
			if (MeshIn->HasAttributes())
			{
				MeshOut.SuspendUVIndexing();

				check(MeshIn->TriangleCount() == MeshOut.Triangles().Num())
				int32 NumLayers = MeshIn->Attributes()->NumUVLayers();			
				MeshOut.SetNumUVChannels(NumLayers); // This resets MeshDescription's internal TriangleUV array
				InstanceAttrib.SetNumChannels(NumLayers);

				for (int UVLayerIndex = 0; UVLayerIndex < NumLayers; UVLayerIndex++)
				{
					const FDynamicMeshUVOverlay* UVOverlay = MeshIn->Attributes()->GetUVLayer(UVLayerIndex);
					// update the vertex Attribute UVs
					DynamicMeshToMeshDescriptionConversionHelper::SetAttributesFromOverlay(MeshIn, MeshOut, InstanceAttrib, UVOverlay, FVector2f::ZeroVector, UVLayerIndex);

					// rebuild the shared UVs
					FUVArray& UVArray = MeshOut.UVs(UVLayerIndex);
					UVArray.Reset(); // delete existing UV buffer
					UVArray.Reserve(UVOverlay->ElementCount());
					
					// rebuild the UV buffer
					int32 MaxID = UVOverlay->MaxElementID(); // the true maxid +1 
					TArray<FUVID> ElIDToUVIDMap;  
					ElIDToUVIDMap.AddUninitialized(MaxID);
					for (int32 ElID = 0; ElID < MaxID; ++ElID)
					{
						if (!UVOverlay->IsElement(ElID))
						{
							continue;
						}

						FVector2f UVValue = UVOverlay->GetElement(ElID);
						FUVID UVID = UVArray.Add();
						ElIDToUVIDMap[ElID] = UVID;
						UVArray.GetAttributes().GetAttributesRef<FVector2f>(MeshAttribute::UV::UVCoordinate)[UVID] = UVValue;
					}

					for (const FTriangleID TriangleID : MeshOut.Triangles().GetElementIDs())
					{
						int TriID = TriangleID.GetValue(); // assumes the same TriIDs in both meshes.
						TArray<FUVID, TFixedAllocator<3>> MDTri;
						if (UVOverlay->IsSetTriangle(TriID))
						{
							FIndex3i ElIDs = UVOverlay->GetTriangle(TriID);
							MDTri.Add(ElIDToUVIDMap[ElIDs[0]]);
							MDTri.Add(ElIDToUVIDMap[ElIDs[1]]);
							MDTri.Add(ElIDToUVIDMap[ElIDs[2]]);
						}
						else
						{
							for (int j = 0; j < 3; ++j)
							{
								FUVID UVID = UVArray.Add();
								UVArray.GetAttributes().GetAttributesRef<FVector2f>(MeshAttribute::UV::UVCoordinate)[UVID] = FVector2f::ZeroVector;
								MDTri.Add(UVID);
							}
						}
						MeshOut.SetTriangleUVIndices(TriangleID, MDTri, UVLayerIndex);
					}
				}

				MeshOut.ResumeUVIndexing();

				#if 0
				{ 
					// Verify the shared UVs and per-vertexinstance UVs match
					for (int UVLayerIndex = 0; UVLayerIndex < NumLayers; UVLayerIndex++)
					{ 
						for (const FTriangleID TriangleID : MeshOut.Triangles().GetElementIDs())
						{
							TArrayView<const FVertexInstanceID> TriWedges = MeshOut.GetTriangleVertexInstances(TriangleID);
							TArrayView<FUVID> UVTri = MeshOut.GetTriangleUVIndices(TriangleID, UVLayerIndex);

							for (int32 i = 0; i < 3; ++i)
							{
								// UV from shared
								FVector2f SharedUV = MeshOut.UVs(UVLayerIndex).GetAttributes().GetAttributesRef<FVector2f>(MeshAttribute::UV::UVCoordinate)[UVTri[i]];
								FVector2f WedgeUV = InstanceAttrib.Get(TriWedges[i], UVLayerIndex);
								check(SharedUV == WedgeUV);
							}
						}
					}
				}
				#endif
			}
			else
			{
				// [todo] correctly build shared UVs?
				check(MeshIn->VertexCount() == MeshOut.Vertices().Num());
				for (int VertID : MeshIn->VertexIndicesItr())
				{
					FVector2f UV = MeshIn->GetVertexUV(VertID);
					for (FVertexInstanceID InstanceID : MeshOut.GetVertexVertexInstanceIDs(FVertexID(VertID)))
					{
						InstanceAttrib.Set(InstanceID, UV);
					}
				}
			}
		}
	}
}


void FDynamicMeshToMeshDescription::UpdateTangents(const FDynamicMesh3* MeshIn, FMeshDescription& MeshOut, const FMeshTangentsd* Tangents)
{
	if (!ensureMsgf(MeshIn->TriangleCount() == MeshOut.Triangles().Num(), TEXT("Trying to update MeshDescription Tangents from Mesh that does not have same triangle count"))) return;
	if (!ensureMsgf(MeshIn->IsCompactT(), TEXT("Trying to update MeshDescription Tangents from a non-compact DynamicMesh"))) return;
	if (!ensureMsgf(MeshIn->HasAttributes(), TEXT("Trying to update MeshDescription Tangents from a DynamicMesh that has no Normals attribute"))) return;

	FStaticMeshAttributes Attributes(MeshOut);

	const FDynamicMeshNormalOverlay* Normals = MeshIn->Attributes()->PrimaryNormals();
	TVertexInstanceAttributesRef<FVector3f> TangentAttrib = Attributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesRef<float> BinormalSignAttrib = Attributes.GetVertexInstanceBinormalSigns();

	if (!ensureMsgf(TangentAttrib.IsValid(), TEXT("Trying to update Tangents on a MeshDescription that has no Tangent Vertex Instance attribute"))) return;
	if (!ensureMsgf(BinormalSignAttrib.IsValid(), TEXT("Trying to update Tangents on a MeshDescription that has no BinormalSign Vertex Instance attribute"))) return;

	if (TangentAttrib.IsValid() && BinormalSignAttrib.IsValid())
	{
		int32 NumTriangles = MeshIn->TriangleCount();
		for (int32 k = 0; k < NumTriangles; ++k)
		{
			FVector3f TriNormals[3];
			Normals->GetTriElements(k, TriNormals[0], TriNormals[1], TriNormals[2]);

			TArrayView<const FVertexInstanceID> TriInstances = MeshOut.GetTriangleVertexInstances(FTriangleID(k));
			for (int j = 0; j < 3; ++j)
			{
				FVector3d Tangent, Bitangent;
				Tangents->GetPerTriangleTangent(k, j, Tangent, Bitangent);
				float BitangentSign = (float)VectorUtil::BitangentSign((FVector3d)TriNormals[j], Tangent, Bitangent);
				TangentAttrib.Set(TriInstances[j], (FVector3f)Tangent);
				BinormalSignAttrib.Set(TriInstances[j], BitangentSign);
			}
		}
	}
}

void FDynamicMeshToMeshDescription::UpdateTangents(const FDynamicMesh3* MeshIn, FMeshDescription& MeshOut)
{
	if (!ensureMsgf(MeshIn->IsCompactT(), TEXT("Trying to update MeshDescription Tangents from a non-compact DynamicMesh"))) return;
	if (!ensureMsgf(MeshIn->TriangleCount() == MeshOut.Triangles().Num(), TEXT("Trying to update MeshDescription Tangents from Mesh that does not have same triangle count"))) return;
	if (!ensureMsgf(MeshIn->HasAttributes(), TEXT("Trying to update MeshDescription Tangents from a DynamicMesh that has no attributes, e.g. normals"))) return;

	// src
	const FDynamicMeshNormalOverlay* NormalOverlay = MeshIn->Attributes()->PrimaryNormals();
	const FDynamicMeshNormalOverlay* TangentOverlay = MeshIn->Attributes()->PrimaryTangents();
	const FDynamicMeshNormalOverlay* BiTangentOverlay = MeshIn->Attributes()->PrimaryBiTangents();

	const bool bHasValidSrc = NormalOverlay && TangentOverlay && BiTangentOverlay;
	ensureMsgf(bHasValidSrc, TEXT("Trying to update MeshDescription Tangents from a DynamicMesh that does not have all three tangent space attributes"));
	if (!bHasValidSrc)
	{
		return;
	}

	// dst
	FStaticMeshAttributes Attributes(MeshOut);
	const TVertexInstanceAttributesRef<FVector3f> TangentAttrib = Attributes.GetVertexInstanceTangents();
	const TVertexInstanceAttributesRef<float> BiTangentSignAttrib = Attributes.GetVertexInstanceBinormalSigns();

	if (!ensureMsgf(TangentAttrib.IsValid(), TEXT("Trying to update Tangents on a MeshDescription that has no Tangent Vertex Instance attribute"))) return;
	if (!ensureMsgf(BiTangentSignAttrib.IsValid(), TEXT("Trying to update Tangents on a MeshDescription that has no BinormalSign Vertex Instance attribute"))) return;

	const int32 NumTriangles = MeshIn->TriangleCount();
	for (int32 t = 0; t < NumTriangles; ++t)
	{
		const bool bTriHasTangentSpace = NormalOverlay->IsSetTriangle(t) && TangentOverlay->IsSetTriangle(t) && BiTangentOverlay->IsSetTriangle(t);

		if (!bTriHasTangentSpace) continue;
		
		// get data from dynamic mesh overlays
		FVector3f TriNormals[3];
		NormalOverlay->GetTriElements(t, TriNormals[0], TriNormals[1], TriNormals[2]);
	

		FVector3f TriTangents[3];
		TangentOverlay->GetTriElements(t, TriTangents[0], TriTangents[1], TriTangents[2]);
		

		FVector3f TriBiTangents[3];
		BiTangentOverlay->GetTriElements(t, TriBiTangents[0], TriBiTangents[1], TriBiTangents[2]);
		
		// set data in the mesh description per-vertex attributes
		TArrayView<const FVertexInstanceID> TriInstances = MeshOut.GetTriangleVertexInstances(FTriangleID(t));
		for (int32 i = 0; i < 3; ++i)
		{
			const float BiTangentSign = VectorUtil::BitangentSign(TriNormals[i], TriTangents[i], TriBiTangents[i]);

			TangentAttrib.Set(TriInstances[i], TriTangents[i]);
			BiTangentSignAttrib.Set(TriInstances[i], BiTangentSign);
		}
	}
}


void FDynamicMeshToMeshDescription::UpdateVertexColors(const FDynamicMesh3* MeshIn, FMeshDescription& MeshOut)
{

	FStaticMeshAttributes Attributes(MeshOut);
	TVertexInstanceAttributesRef<FVector4f> InstanceColorsAttrib = Attributes.GetVertexInstanceColors();

	if (!ensureMsgf(MeshIn->IsCompactT(), TEXT("Trying to update MeshDescription Colors from a non-compact DynamicMesh"))) return;
	if (!ensureMsgf(MeshIn->TriangleCount() == MeshOut.Triangles().Num(), TEXT("Trying to update MeshDescription Colors from Mesh that does not have same triangle count"))) return;
	if (!ensureMsgf(MeshIn->HasAttributes() && MeshIn->Attributes()->HasPrimaryColors(), TEXT("Trying to update MeshDescription Colors from a DynamicMesh that has no attribute set at all or has no color data in its attribute set"))) return;
	if (!ensureMsgf(InstanceColorsAttrib.IsValid(), TEXT("Trying to update colors on a MeshDescription that has no color attributes"))) return;
	
	const FDynamicMeshColorOverlay* ColorOverlay = MeshIn->Attributes()->PrimaryColors();
	
	const int32 NumTriangles = MeshIn->TriangleCount();
	for (int32 t = 0; t < NumTriangles; ++t)
	{
		if (!ColorOverlay->IsSetTriangle(t))
		{
			continue;
		}
	
		FVector4f TriColors[3];
		ColorOverlay->GetTriElements(t, TriColors[0], TriColors[1], TriColors[2]);

		ApplyVertexColorTransform(TriColors[0]);
		ApplyVertexColorTransform(TriColors[1]);
		ApplyVertexColorTransform(TriColors[2]);
		
		TArrayView<const FVertexInstanceID> InstanceIDs = MeshOut.GetTriangleVertexInstances(FTriangleID(t));
		for (int32 i = 0; i < 3; ++i)
		{
			FVector4f InstanceColor4(TriColors[i].X, TriColors[i].Y, TriColors[i].Z, TriColors[i].W);
			InstanceColorsAttrib.Set(InstanceIDs[i], 0, InstanceColor4);
		}

	}
}




void FDynamicMeshToMeshDescription::Convert(const FDynamicMesh3* MeshIn, FMeshDescription& MeshOut, bool bCopyTangents)
{
	if (MeshIn->HasAttributes())
	{
		//Convert_SharedInstances(MeshIn, MeshOut);
		Convert_NoSharedInstances(MeshIn, MeshOut, bCopyTangents);
	}
	else
	{
		Convert_NoAttributes(MeshIn, MeshOut);
	}
}


void FDynamicMeshToMeshDescription::SetMaterialIDMapFromInverseMap(TArrayView<const int32> PolygroupIDToMaterialIDMap)
{
	int32 MaxMatID = -1;
	for (int32 MatID : PolygroupIDToMaterialIDMap)
	{
		MaxMatID = FMath::Max(MaxMatID, MatID);
	}
	MaterialIDToPolygroupIDMap.SetNumUninitialized(MaxMatID + 1);
	for (int32 Idx = 0; Idx < MaterialIDToPolygroupIDMap.Num(); ++Idx)
	{
		MaterialIDToPolygroupIDMap[Idx] = Idx;
	}
	for (int32 PolyGroupIdx = 0; PolyGroupIdx < PolygroupIDToMaterialIDMap.Num(); ++PolyGroupIdx)
	{
		int32 MatID = PolygroupIDToMaterialIDMap[PolyGroupIdx];
		if (MaterialIDToPolygroupIDMap.IsValidIndex(MatID))
		{
			MaterialIDToPolygroupIDMap[MatID] = PolyGroupIdx;
		}
	}
}


bool FDynamicMeshToMeshDescription::HaveMatchingElementCounts(const FDynamicMesh3* DynamicMesh, const FMeshDescription* MeshDescription, bool bVerticesOnly, bool bAttributesOnly)
{
	bool bVerticesMatch = DynamicMesh->IsCompactV() && DynamicMesh->VertexCount() == MeshDescription->Vertices().Num();
	bool bTrianglesMatch = DynamicMesh->IsCompactT() && DynamicMesh->TriangleCount() == MeshDescription->Triangles().Num();
	if (bVerticesOnly || (bAttributesOnly && !DynamicMesh->HasAttributes()))
	{
		return bVerticesMatch;
	}
	else if (bAttributesOnly && DynamicMesh->HasAttributes())
	{
		return bTrianglesMatch;
	}
	return bVerticesMatch && bTrianglesMatch;
}


bool FDynamicMeshToMeshDescription::HaveMatchingElementCounts(const FDynamicMesh3* DynamicMesh, const FMeshDescription* MeshDescription)
{
	bool bUpdateAttributes = ConversionOptions.bUpdateNormals || ConversionOptions.bUpdateUVs;
	return HaveMatchingElementCounts(DynamicMesh, MeshDescription, !bUpdateAttributes, !ConversionOptions.bUpdatePositions);
}


void FDynamicMeshToMeshDescription::UpdateUsingConversionOptions(const FDynamicMesh3* MeshIn, FMeshDescription& MeshOut)
{
	// See if we can do a buffer update without having to alter triangles.
	if (HaveMatchingElementCounts(MeshIn, &MeshOut))
	{
		if (ConversionOptions.bUpdatePositions)
		{
			Update(MeshIn, MeshOut, ConversionOptions.bUpdateNormals, ConversionOptions.bUpdateTangents, ConversionOptions.bUpdateUVs);
		}
		else if (ConversionOptions.bUpdateNormals || ConversionOptions.bUpdateTangents || ConversionOptions.bUpdateUVs)
		{
			UpdateAttributes(MeshIn, MeshOut, ConversionOptions.bUpdateNormals, ConversionOptions.bUpdateTangents, ConversionOptions.bUpdateUVs);
		}

		if (ConversionOptions.bUpdateVtxColors)
		{
			UpdateVertexColors(MeshIn, MeshOut);
		}
	}
	else
	{
		// Do a full conversion.
		Convert(MeshIn, MeshOut);
	}
}


void FDynamicMeshToMeshDescription::Convert_NoAttributes(const FDynamicMesh3* MeshIn, FMeshDescription& MeshOut)
{
	MeshOut.Empty();

	FMeshDescriptionBuilder Builder;
	Builder.SetMeshDescription(&MeshOut);

	Builder.SuspendMeshDescriptionIndexing();
	constexpr int32 UVLayerIndex = 0;
	Builder.SetNumUVLayers(1);
	Builder.ReserveNewUVs(MeshIn->VertexCount(), UVLayerIndex);

	bool bCopyGroupToPolyGroup = false;
	if (ConversionOptions.bSetPolyGroups && MeshIn->HasTriangleGroups())
	{
		Builder.EnablePolyGroups();
		bCopyGroupToPolyGroup = true;
	}

	// Convert the vertices and construct a mapping table between the dynamic mesh and the MeshDescription, optionally applying
	// an undoing conversion for the non-manifold fixes applied when converting from a MeshDescription to dynamic mesh.
	TArray<FVertexID> MapV;
	DynamicMeshToMeshDescriptionConversionHelper::ConvertVertices(*MeshIn, Builder, ConversionOptions.bConvertBackToNonManifold, MapV);
	
	FPolygonGroupID AllGroupID = Builder.AppendPolygonGroup();

	// create new instances when seen
	TMap<FIndex3i, FVertexInstanceID> InstanceList;
	TMap<int32, FUVID> InstanceUVIDMap;
	for (int TriID : MeshIn->TriangleIndicesItr())
	{
		FIndex3i Triangle = MeshIn->GetTriangle(TriID);
		FIndex3i UVTriangle(-1, -1, -1);
		FIndex3i NormalTriangle = Triangle;
		FVertexInstanceID InstanceTri[3];
		FUVID UVIDs[3];
		for (int j = 0; j < 3; ++j)
		{
			FIndex3i InstanceElem(Triangle[j], UVTriangle[j], NormalTriangle[j]);
			if (InstanceList.Contains(InstanceElem) == false)
			{
				FVertexInstanceID NewInstanceID = Builder.AppendInstance(MapV[Triangle[j]]);
				InstanceList.Add(InstanceElem, NewInstanceID);
				
				FVector Normal = MeshIn->HasVertexNormals() ? FVector(MeshIn->GetVertexNormal(Triangle[j])) : FVector::UpVector;
				Builder.SetInstanceNormal(NewInstanceID, Normal);

				// Add UV to MeshDescription UVvertex buffer
				FVector2D UV = MeshIn->HasVertexUVs() ? FVector2D(MeshIn->GetVertexUV(Triangle[j])) : FVector2D::ZeroVector;
				FUVID UVID = Builder.AppendUV(UV, UVLayerIndex);
				
				// associate UVID with this instance
				InstanceUVIDMap.Add(NewInstanceID.GetValue(), UVID);				
			}
			InstanceTri[j] = InstanceList[InstanceElem];
			UVIDs[j] = InstanceUVIDMap[InstanceTri[j].GetValue()];
		}

		FTriangleID NewTriangleID = Builder.AppendTriangle(InstanceTri[0], InstanceTri[1], InstanceTri[2], AllGroupID);
		
		// append the UV triangle - builder takes care of the rest
		Builder.AppendUVTriangle(NewTriangleID, UVIDs[0], UVIDs[1], UVIDs[2], UVLayerIndex);
		
		if (bCopyGroupToPolyGroup)
		{
			Builder.SetPolyGroupID(NewTriangleID, MeshIn->GetTriangleGroup(TriID));
		}
	}

	Builder.ResumeMeshDescriptionIndexing();
}







void FDynamicMeshToMeshDescription::Convert_SharedInstances(const FDynamicMesh3* MeshIn, FMeshDescription& MeshOut)
{
	const FDynamicMeshNormalOverlay* NormalOverlay = MeshIn->HasAttributes() ? MeshIn->Attributes()->PrimaryNormals() : nullptr;

	MeshOut.Empty();

	FMeshDescriptionBuilder Builder;
	Builder.SetMeshDescription(&MeshOut);

	bool bCopyGroupToPolyGroup = false;
	if (ConversionOptions.bSetPolyGroups && MeshIn->HasTriangleGroups())
	{
		Builder.EnablePolyGroups();
		bCopyGroupToPolyGroup = true;
	}

	// create vertices
	TArray<FVertexID> MapV; MapV.SetNum(MeshIn->MaxVertexID());
	for (int VertID : MeshIn->VertexIndicesItr())
	{
		MapV[VertID] = Builder.AppendVertex((FVector)MeshIn->GetVertex(VertID));
	}


	FPolygonGroupID ZeroPolygonGroupID = Builder.AppendPolygonGroup();

	// check if we have per-triangle material ID
	const FDynamicMeshMaterialAttribute* MaterialIDAttrib =
		(MeshIn->HasAttributes() && MeshIn->Attributes()->HasMaterialID()) ?
		MeshIn->Attributes()->GetMaterialID() : nullptr;

	// need to know max material index value so we can reserve groups in MeshDescription
	int32 MaxPolygonGroupID = 0;
	if (MaterialIDAttrib)
	{
		for (int TriID : MeshIn->TriangleIndicesItr())
		{
			int32 MaterialID;
			MaterialIDAttrib->GetValue(TriID, &MaterialID);
			if (MaterialIDToPolygroupIDMap.IsValidIndex(MaterialID))
			{
				MaterialID = MaterialIDToPolygroupIDMap[MaterialID];
			}
			DynamicMeshToMeshDescriptionConversionHelper::ValidateMaterialID(MaterialID, ConversionOptions.MaxValidMaterialID);
			MaxPolygonGroupID = FMath::Max(MaterialID, MaxPolygonGroupID);
		}
		if (MaxPolygonGroupID == 0)
		{
			MaterialIDAttrib = nullptr;
		}
		else
		{
			for (int k = 0; k < MaxPolygonGroupID; ++k)
			{
				Builder.AppendPolygonGroup();
			}
		}
	}

	// build all vertex instances (splitting as needed)
	// store per-triangle instance ids
	struct Tri
	{
		FVertexInstanceID V[3];

		Tri() : V{ INDEX_NONE,INDEX_NONE,INDEX_NONE }
		{}
	};
	TArray<Tri> TriVertInstances;
	TriVertInstances.SetNum(MeshIn->MaxTriangleID());
	TArray<int32> KnownInstanceIDs;
	int NumUVLayers = MeshIn->HasAttributes() ? MeshIn->Attributes()->NumUVLayers() : 0;
	Builder.SetNumUVLayers(NumUVLayers);
	int KIItemLen = 1 + (NormalOverlay ? 1 : 0) + NumUVLayers;
	for (int VertID : MeshIn->VertexIndicesItr())
	{
		KnownInstanceIDs.Reset();
		for (int TriID : MeshIn->VtxTrianglesItr(VertID))
		{
			FIndex3i Tri = MeshIn->GetTriangle(TriID);
			int SubIdx = IndexUtil::FindTriIndex(VertID, Tri);

			int32 FoundInstance = INDEX_NONE;
			for (int KIItemIdx = 0; KIItemIdx < KnownInstanceIDs.Num(); KIItemIdx += KIItemLen)
			{
				int KIItemInternalIdx = KIItemIdx;

				if (NormalOverlay && KnownInstanceIDs[KIItemInternalIdx++] != NormalOverlay->GetTriangle(TriID)[SubIdx])
				{
					continue;
				}

				bool FoundInUVs = true;
				for (int UVLayerIndex = 0; UVLayerIndex < NumUVLayers; UVLayerIndex++)
				{
					const FDynamicMeshUVOverlay* Overlay = MeshIn->Attributes()->GetUVLayer(UVLayerIndex);
					if (KnownInstanceIDs[KIItemInternalIdx++] != Overlay->GetTriangle(TriID)[SubIdx])
					{
						FoundInUVs = false;
						break;
					}
				}
				if (!FoundInUVs)
				{
					continue;
				}

				FoundInstance = KnownInstanceIDs[KIItemInternalIdx++];
				check(KIItemInternalIdx == KIItemIdx + KIItemLen);
				break;
			}
			if (FoundInstance == INDEX_NONE)
			{
				FVertexInstanceID NewInstanceID = Builder.AppendInstance(MapV[VertID]);
				if (NormalOverlay)
				{
					int ElID = NormalOverlay->GetTriangle(TriID)[SubIdx];
					KnownInstanceIDs.Add(int32(ElID));
					FVector3f ElementNormal = ElID != -1 ? NormalOverlay->GetElement(ElID) : FVector3f::UnitZ();
					Builder.SetInstanceNormal(NewInstanceID, (FVector)ElementNormal);
				}
				else
				{
					Builder.SetInstanceNormal(NewInstanceID, FVector::UpVector);
				}
				for (int UVLayerIndex = 0; UVLayerIndex < NumUVLayers; UVLayerIndex++)
				{
					const FDynamicMeshUVOverlay* Overlay = MeshIn->Attributes()->GetUVLayer(UVLayerIndex);
					int ElID = Overlay->GetTriangle(TriID)[SubIdx];
					KnownInstanceIDs.Add(int32(ElID));

					FVector2f ElementUV = ElID != -1 ? Overlay->GetElement(ElID) : FVector2f::Zero();
					Builder.SetInstanceUV(NewInstanceID, (FVector2D)ElementUV, UVLayerIndex);
				}
				FoundInstance = NewInstanceID.GetValue();
				KnownInstanceIDs.Add(FoundInstance);
			}
			TriVertInstances[TriID].V[SubIdx] = FVertexInstanceID(FoundInstance);
		}
	}

	// build the polygons
	for (int TriID : MeshIn->TriangleIndicesItr())
	{
		// transfer material index to MeshDescription polygon group (by convention)
		FPolygonGroupID UsePolygonGroupID = ZeroPolygonGroupID;
		if (MaterialIDAttrib)
		{
			int32 MaterialID;
			MaterialIDAttrib->GetValue(TriID, &MaterialID);
			if (MaterialIDToPolygroupIDMap.IsValidIndex(MaterialID))
			{
				MaterialID = MaterialIDToPolygroupIDMap[MaterialID];
			}
			MaterialID = FMath::Max(0, MaterialID);
			DynamicMeshToMeshDescriptionConversionHelper::ValidateMaterialID(MaterialID, ConversionOptions.MaxValidMaterialID);
			UsePolygonGroupID = FPolygonGroupID(MaterialID);
		}

		FTriangleID NewTriangleID = Builder.AppendTriangle(TriVertInstances[TriID].V[0], TriVertInstances[TriID].V[1], TriVertInstances[TriID].V[2], UsePolygonGroupID);

		if (bCopyGroupToPolyGroup)
		{
			Builder.SetPolyGroupID(NewTriangleID, MeshIn->GetTriangleGroup(TriID));
		}
	}
}




void FDynamicMeshToMeshDescription::Convert_NoSharedInstances(const FDynamicMesh3* MeshIn, FMeshDescription& MeshOut, bool bCopyTangents)
{
	
	const bool bHasAttributes = MeshIn->HasAttributes();

	// check if we have per-triangle material ID
	const FDynamicMeshMaterialAttribute* MaterialIDAttrib = (bHasAttributes && MeshIn->Attributes()->HasMaterialID()) ? MeshIn->Attributes()->GetMaterialID() : nullptr;

	// cache tangent space and UV and color overlay info
	const FDynamicMeshNormalOverlay* NormalOverlay = bHasAttributes ? MeshIn->Attributes()->PrimaryNormals() : nullptr;
	const FDynamicMeshNormalOverlay* TangentOverlay = bHasAttributes ? MeshIn->Attributes()->PrimaryTangents() : nullptr;
	const FDynamicMeshNormalOverlay* BiTangentOverlay = bHasAttributes ? MeshIn->Attributes()->PrimaryBiTangents() : nullptr;
	const FDynamicMeshColorOverlay* ColorOverlay = bHasAttributes ? MeshIn->Attributes()->PrimaryColors() : nullptr;

	const int32 NumUVLayers = bHasAttributes ? MeshIn->Attributes()->NumUVLayers() : 0;
	
	const bool bHasBones = bHasAttributes && MeshIn->Attributes()->HasBones();
	const FDynamicMeshBoneNameAttribute* BoneNames = bHasBones ? MeshIn->Attributes()->GetBoneNames() : nullptr;
	const FDynamicMeshBoneParentIndexAttribute* BoneParentIndices = bHasBones ? MeshIn->Attributes()->GetBoneParentIndices() : nullptr;
	const FDynamicMeshBonePoseAttribute* BonePoses = bHasBones ? MeshIn->Attributes()->GetBonePoses() : nullptr;
	const FDynamicMeshBoneColorAttribute* BoneColors = bHasBones ? MeshIn->Attributes()->GetBoneColors() : nullptr;

	const bool bHasSkinWeightsAttributes = bHasAttributes && !MeshIn->Attributes()->GetSkinWeightsAttributes().IsEmpty();
	const bool bHasMorphTargetAttributes = bHasAttributes && !MeshIn->Attributes()->GetMorphTargetAttributes().IsEmpty();

	// cache the UV layers
	TArray<const FDynamicMeshUVOverlay*> UVLayers;
	for (int32 k = 0; k < NumUVLayers; ++k)
	{
		UVLayers.Add(MeshIn->Attributes()->GetUVLayer(k));
	}


	MeshOut.Empty();

	FMeshDescriptionBuilder Builder;
	Builder.SetMeshDescription(&MeshOut);

	// Make sure edge hardnesses are registered, if they might be set below
	if (ConversionOptions.bSetHardEdgesFromNormalSeams && NormalOverlay)
	{
		FStaticMeshAttributes MeshOutAttributes(MeshOut);
		MeshOutAttributes.Register();
	}

	bool bCopyGroupToPolyGroup = false;
	if (ConversionOptions.bSetPolyGroups && MeshIn->HasTriangleGroups())
	{
		Builder.EnablePolyGroups();
		bCopyGroupToPolyGroup = true;
	}

	// We register skeletal attributes if either bone names or skinning information is available
	if (bHasBones || bHasSkinWeightsAttributes)
	{
		FSkeletalMeshAttributes MeshOutAttributes(MeshOut);
		MeshOutAttributes.Register();
		MeshOutAttributes.RegisterColorAttribute();
		
		// If there was an attribute to map from the mesh description index to the imported mesh index, remove that
		// now, since we don't carry this mapping over through the dynamic mesh.
		MeshOutAttributes.UnregisterImportPointIndexAttribute();
	}

	if (bHasBones)
	{
		FSkeletalMeshAttributes MeshOutAttributes(MeshOut);
		checkSlow(MeshOutAttributes.HasBones()); // FSkeletalMeshAttributes::Register() must have been called

		
		
		const int32 NumBones = MeshIn->Attributes()->GetNumBones();

		MeshOutAttributes.BoneAttributes();
		MeshOutAttributes.ReserveNewBones(NumBones);

		for (int Idx = 0; Idx < NumBones; ++Idx)
		{
			const FBoneID BoneID = MeshOutAttributes.CreateBone();
			MeshOutAttributes.GetBoneNames().Set(BoneID, BoneNames->GetValue(Idx));
			MeshOutAttributes.GetBoneParentIndices().Set(BoneID, BoneParentIndices->GetValue(Idx));
			MeshOutAttributes.GetBonePoses().Set(BoneID, BonePoses->GetValue(Idx));
			MeshOutAttributes.GetBoneColors().Set(BoneID, BoneColors->GetValue(Idx));
		}
	}

	TMap<FName, FSkinWeightsVertexAttributesRef> VertexBoneWeightsMap;
	if (bHasSkinWeightsAttributes)
	{
		FSkeletalMeshAttributes MeshOutAttributes(MeshOut);
		
		for (const FName Name : MeshOutAttributes.GetSkinWeightProfileNames())
		{
			MeshOutAttributes.UnregisterSkinWeightAttribute(Name);
		}
		
		for (const TTuple<FName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>>& AttributeInfo: MeshIn->Attributes()->GetSkinWeightsAttributes())
		{
			FName ProfileName = AttributeInfo.Key;

			MeshOutAttributes.RegisterSkinWeightAttribute(ProfileName);
			VertexBoneWeightsMap.Add(ProfileName, MeshOutAttributes.GetVertexSkinWeights(ProfileName));
		}
	}

	TMap<FName, TVertexAttributesRef<FVector3f>> MorphTargetsMap;
	if (bHasMorphTargetAttributes)
	{
		FSkeletalMeshAttributes MeshOutAttributes(MeshOut);
		
		for (const FName Name : MeshOutAttributes.GetMorphTargetNames())
		{
			MeshOutAttributes.UnregisterMorphTargetAttribute(Name);
		}
		
		for (const TTuple<FName, TUniquePtr<FDynamicMeshMorphTargetAttribute>>& AttributeInfo: MeshIn->Attributes()->GetMorphTargetAttributes())
		{
			FName MorphTargetName = AttributeInfo.Key;

			MeshOutAttributes.RegisterMorphTargetAttribute(MorphTargetName, false);
			MorphTargetsMap.Add(MorphTargetName, MeshOutAttributes.GetVertexMorphPositionDelta(MorphTargetName));
		}
	}

	// always copy when we are baking new mesh? should this be a config option?
	bool bCopyInstanceColors = (ColorOverlay != nullptr); 

	// disable indexing during the full build of the mesh
	Builder.SuspendMeshDescriptionIndexing();

	// Convert the vertices and construct a mapping table between the dynamic mesh and the MeshDescription, optionally applying
	// an undoing conversion for the non-manifold fixes applied when converting from a MeshDescription to dynamic mesh.
	TArray<FVertexID> MapV;
	DynamicMeshToMeshDescriptionConversionHelper::ConvertVertices(*MeshIn, Builder, ConversionOptions.bConvertBackToNonManifold, MapV);

	// create UV vertex buffer in MeshDescription
	Builder.SetNumUVLayers(NumUVLayers);

	TArray<TArray<FUVID>> MapUVArray; MapUVArray.SetNum(NumUVLayers);
	TArray<FUVID> UnsetUVIDs; UnsetUVIDs.Init(INDEX_NONE, NumUVLayers); // UVIDs for unset UVs to share, if needed
	for (int32 k = 0; k < NumUVLayers; ++k)
	{
		const FDynamicMeshUVOverlay* UVOverlay = UVLayers[k];
		TArray<FUVID>& MapUV = MapUVArray[k];

		MapUV.SetNum(UVOverlay->MaxElementID());
		Builder.ReserveNewUVs(UVOverlay->ElementCount(), k); 

		for (int32 ElementID : UVOverlay->ElementIndicesItr())
		{
			const FVector2D UVvalue = (FVector2D)UVOverlay->GetElement(ElementID); 
			MapUV[ElementID] = Builder.AppendUV(UVvalue, k);
		}
	}


	FPolygonGroupID ZeroPolygonGroupID = Builder.AppendPolygonGroup();

	
	// construct a function that will transfer tangent space data. 
	// if the DynamicMesh has a full tangent space: a normal, tangent and bitangent sign will be transfered
	//                                            otherwise just transfer the normal if it exists
	const bool bCopyFullTangentSpace = bCopyTangents && (NormalOverlay != nullptr && TangentOverlay != nullptr && BiTangentOverlay != nullptr);

	TFunction<void(int TriID, FVertexInstanceID TriVertInstances[3])> TangetSpaceInstanceSetter;
	if (bCopyFullTangentSpace) // create function that sets the tangent space
	{
		TangetSpaceInstanceSetter = [NormalOverlay, TangentOverlay, BiTangentOverlay, &Builder](int TriID, FVertexInstanceID TriVertInstances[3])
		{
			FIndex3i NormalTri = NormalOverlay->GetTriangle(TriID);
			FIndex3i TangentTri = TangentOverlay->GetTriangle(TriID);
			FIndex3i BiTangentTri = BiTangentOverlay->GetTriangle(TriID);

			for (int32 j = 0; j < 3; ++j)
			{
				const FVertexInstanceID CornerInstanceID = TriVertInstances[j];
				
				const FVector3f TriVertNormal = (NormalOverlay->IsElement(NormalTri[j]))  ? NormalOverlay->GetElement(NormalTri[j]) : FVector3f(FVector::UpVector);
				const FVector3f TriVertTangent = (TangentOverlay->IsElement(TangentTri[j])) ? TangentOverlay->GetElement(TangentTri[j]) :  FVector3f(FVector::ForwardVector);
				const FVector3f TriVertBiTangent = (BiTangentOverlay->IsElement(BiTangentTri[j])) ? BiTangentOverlay->GetElement(BiTangentTri[j]) : FVector3f(FVector::RightVector);

				// infer sign
				float BiTangentSign = VectorUtil::BitangentSign(TriVertNormal, TriVertTangent, TriVertBiTangent);

				// set the tangent space
				Builder.SetInstanceTangentSpace(CornerInstanceID, (FVector)TriVertNormal, (FVector)TriVertTangent, BiTangentSign);
			}
			
		};
	}
	else if (NormalOverlay != nullptr) // create function that just sets the normals
	{
		TangetSpaceInstanceSetter = [NormalOverlay, &Builder](int TriID, FVertexInstanceID TriVertInstances[3])
		{
			FIndex3i NormalTri = NormalOverlay->GetTriangle(TriID);
			for (int32 j = 0; j < 3; ++j)
			{
				FVertexInstanceID CornerInstanceID = TriVertInstances[j];
				FVector TriVertNormal = (NormalOverlay->IsElement(NormalTri[j])) ? (FVector)NormalOverlay->GetElement(NormalTri[j]) : FVector::UpVector;
				Builder.SetInstanceNormal(CornerInstanceID, TriVertNormal);
			}
		};
	}
	else  // dummy function that does nothing 
	{
		TangetSpaceInstanceSetter = [](int TriID, FVertexInstanceID TriVertInstances[3]){ return;};
	}
	TFunction<void(int TriID, FVertexInstanceID TriVertInstances[3])> ColorInstanceSetter;
	if (bCopyInstanceColors)
	{
		ColorInstanceSetter = [this, ColorOverlay, &Builder](int TriID, FVertexInstanceID TriVertInstances[3])
		{
			FIndex3i ColorTri = ColorOverlay->GetTriangle(TriID);
			for (int32 j = 0; j < 3; ++j)
			{
				FVector4f DstColor(1,1,1,1);
				const FVertexInstanceID CornerInstanceID = TriVertInstances[j];
				if (ColorOverlay->IsElement(ColorTri[j]))
				{
					FVector4f TriVertColor4 = ColorOverlay->GetElement(ColorTri[j]);
					ApplyVertexColorTransform(TriVertColor4);
					DstColor = FVector4f(TriVertColor4.X, TriVertColor4.Y, TriVertColor4.Z, TriVertColor4.W);
				}
				Builder.SetInstanceColor(CornerInstanceID, DstColor);
			}
		};
	}
	else
	{
		ColorInstanceSetter = [](int TriID, FVertexInstanceID TriVertInstances[3]) { return; };
	}

	// need to know max material index value so we can reserve groups in MeshDescription
	int32 MaxPolygonGroupID = 0;
	if (MaterialIDAttrib)
	{
		for (int TriID : MeshIn->TriangleIndicesItr())
		{
			int32 MaterialID;
			MaterialIDAttrib->GetValue(TriID, &MaterialID);
			if (MaterialIDToPolygroupIDMap.IsValidIndex(MaterialID))
			{
				MaterialID = MaterialIDToPolygroupIDMap[MaterialID];
			}
			DynamicMeshToMeshDescriptionConversionHelper::ValidateMaterialID(MaterialID, ConversionOptions.MaxValidMaterialID);
			MaxPolygonGroupID = FMath::Max(MaterialID, MaxPolygonGroupID);
		}
		if (MaxPolygonGroupID == 0)
		{
			MaterialIDAttrib = nullptr;
		}
		else
		{
			for (int k = 0; k < MaxPolygonGroupID; ++k)
			{
				Builder.AppendPolygonGroup();
			}
		}
	}

	

	TArray<FIndex3i> UVTris;
	UVTris.SetNum(NumUVLayers);

	TArray<FTriangleID> IndexToTriangleIDMap;
	IndexToTriangleIDMap.SetNum(MeshIn->MaxTriangleID());

	for (int TriID : MeshIn->TriangleIndicesItr())
	{
		FIndex3i Triangle = MeshIn->GetTriangle(TriID);

		// create new vtx instances for each triangle
		FVertexInstanceID TriVertInstances[3];
		for (int32 j = 0; j < 3; ++j)
		{
			const FVertexID TriVertex = MapV[Triangle[j]];
			TriVertInstances[j] = Builder.AppendInstance(TriVertex);
		}


		// transfer material index to MeshDescription polygon group (by convention)
		FPolygonGroupID UsePolygonGroupID = ZeroPolygonGroupID;
		if (MaterialIDAttrib)
		{
			int32 MaterialID;
			MaterialIDAttrib->GetValue(TriID, &MaterialID);
			if (MaterialIDToPolygroupIDMap.IsValidIndex(MaterialID))
			{
				MaterialID = MaterialIDToPolygroupIDMap[MaterialID];
			}
			MaterialID = FMath::Max(0, MaterialID);
			DynamicMeshToMeshDescriptionConversionHelper::ValidateMaterialID(MaterialID, ConversionOptions.MaxValidMaterialID);
			UsePolygonGroupID = FPolygonGroupID(MaterialID);
		}

		// add the triangle to MeshDescription
		FTriangleID NewTriangleID = Builder.AppendTriangle(TriVertInstances[0], TriVertInstances[1], TriVertInstances[2], UsePolygonGroupID); 
		IndexToTriangleIDMap[TriID] = NewTriangleID;

		// transfer UVs.  Note the Builder sets both the shared and per-instance UVs from this
		for (int32 k = 0; k < NumUVLayers; ++k)
		{ 
			FUVID UVIDs[3] = {FUVID(-1), FUVID(-1), FUVID(-1)};

			// add zero uvs for unset triangles.
			if (!UVLayers[k]->IsSetTriangle(TriID))
			{
				if (UnsetUVIDs[k].GetValue() == INDEX_NONE)
				{
					UnsetUVIDs[k] = Builder.AppendUV(FVector2D::ZeroVector, k);
				}
				for (int32 j = 0; j < 3; ++j)
				{
					UVIDs[j] = UnsetUVIDs[k];
				}
			}
			else
			{ 
				const TArray<FUVID>& MapUV = MapUVArray[k];
		
				// triangle of UV element ids from dynamic mesh.  references values already stored in MeshDescription.
				FIndex3i UVTri = UVLayers[k]->GetTriangle(TriID);

				// translate to MeshDescription Ids
				for (int32 j = 0; j < 3; ++j)
				{ 
					UVIDs[j] = MapUV[ UVTri[j] ];
				}
			}
			
			// append the UV triangle - builder takes care of the rest
			Builder.AppendUVTriangle(NewTriangleID, UVIDs[0], UVIDs[1], UVIDs[2], k); 

		}
		
		// transfer tangent space. 
		// NB: MeshDescription doesn't store and explicit bitangent, so this conversion isn't perfect.
		// NB: only per-instance normals , tangents, bitangent sign are supported in MeshDescription at this time
		// NB: will need to be updated to follow the pattern used in UVs above when MeshDescription supports shared tangent space elements. 
		TangetSpaceInstanceSetter(TriID, TriVertInstances);

		// transfer the color overlay to per-instance colors.
		// NB: will need to be updated to follow the pattern used in UVs above if MeshDescription supports shared color elements
		ColorInstanceSetter(TriID, TriVertInstances);


		if (bCopyGroupToPolyGroup)
		{
			Builder.SetPolyGroupID(NewTriangleID, MeshIn->GetTriangleGroup(TriID));
		}
	}

	// convert polygroup layers
	ConvertPolygroupLayers(MeshIn, MeshOut, IndexToTriangleIDMap);

	// convert weight map layers
	ConvertWeightLayers(MeshIn, MeshOut, MapV);
	
	// convert triangle attributes
	ConvertTriangleAttributes(MeshIn, MeshOut, IndexToTriangleIDMap);

	// convert normal seams to hard edges
	if (ConversionOptions.bSetHardEdgesFromNormalSeams && NormalOverlay)
	{
		FStaticMeshAttributes MeshOutAttributes(MeshOut);
		TEdgeAttributesRef<bool> EdgeHardness = MeshOutAttributes.GetEdgeHardnesses();

		// Find seam edges on the dynamic mesh
		TArray<bool> MeshSeamEdges;
		MeshSeamEdges.SetNumZeroed(MeshIn->MaxEdgeID());
		ParallelFor(MeshIn->MaxEdgeID(), [&MeshIn, &NormalOverlay, &MeshSeamEdges](int32 EID)
			{
				if (MeshIn->IsEdge(EID))
				{
					MeshSeamEdges[EID] = NormalOverlay->IsSeamEdge(EID);
				}
			});

		// For any seam edge, find the corresponding mesh description edge and tag it as hard
		// Note that this will also tag non-manifold edges (if preserved via remapping) as hard edges
		for (int32 EID = 0; EID < MeshSeamEdges.Num(); ++EID)
		{
			if (MeshSeamEdges[EID]) // will only be true for valid edges that are boundaries or seams
			{
				int32 MeshInTID = MeshIn->GetEdgeT(EID).A;
				int32 EdgeTriSubIdx = MeshIn->GetTriEdges(MeshInTID).IndexOf(EID);
				if (!ensure(EdgeTriSubIdx != IndexConstants::InvalidID))
				{
					continue;
				}

				FTriangleID MeshOutTID = IndexToTriangleIDMap[MeshInTID];
				TArrayView<FEdgeID> OutTriEdges = MeshOut.GetTriangleEdges(MeshOutTID);
				if (!ensure(OutTriEdges.IsValidIndex(EdgeTriSubIdx)))
				{
					continue;
				}
				EdgeHardness[OutTriEdges[EdgeTriSubIdx]] = true;
			}
		}
	}

	// Convert all attached skin weights, if we're converting a mesh description that originated from
	// a USkeletalMesh.
	if (bHasSkinWeightsAttributes)
	{
		for (const TTuple<FName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>>& AttributeInfo: MeshIn->Attributes()->GetSkinWeightsAttributes())
		{
			FName ProfileName = AttributeInfo.Key;
		
			FSkinWeightsVertexAttributesRef& VertexBoneWeights = VertexBoneWeightsMap[ProfileName];

			const FDynamicMeshVertexSkinWeightsAttribute *MeshSkinWeights = AttributeInfo.Value.Get();
			for (int32 VertexIndex = 0; VertexIndex < MapV.Num(); VertexIndex++)
			{
				if (const FVertexID VertexID = MapV[VertexIndex]; VertexID != INDEX_NONE)
				{
					UE::AnimationCore::FBoneWeights BW;
					MeshSkinWeights->GetValue(VertexIndex, BW);
					VertexBoneWeights.Set(VertexID, BW);
				}
			}
		}
	}

	if (bHasMorphTargetAttributes)
	{
		for (const TTuple<FName, TUniquePtr<FDynamicMeshMorphTargetAttribute>>& AttributeInfo: MeshIn->Attributes()->GetMorphTargetAttributes())
		{
			FName ProfileName = AttributeInfo.Key;
		
			TVertexAttributesRef<FVector3f>& MeshDescMorphTarget = MorphTargetsMap[ProfileName];

			const FDynamicMeshMorphTargetAttribute* DynaMeshMorphTarget = AttributeInfo.Value.Get();
			
			for (int32 VertexIndex = 0; VertexIndex < MapV.Num(); VertexIndex++)
			{
				if (const FVertexID VertexID = MapV[VertexIndex]; VertexID != INDEX_NONE)
				{
					FVector3f Offset;
					DynaMeshMorphTarget->GetValue(VertexIndex, Offset);
					MeshDescMorphTarget.Set(VertexID, Offset);	
				}
			}
		}
	}
	
	Builder.ResumeMeshDescriptionIndexing();
}


void FDynamicMeshToMeshDescription::ConvertPolygroupLayers(const FDynamicMesh3* MeshIn, FMeshDescription& MeshOut, const TArray<FTriangleID>& IndexToTriangleIDMap)
{
	TAttributesSet<FTriangleID>& TriAttribsSet = MeshOut.TriangleAttributes();

	TSet<FName> SourceLayerNames;
	const int32 NumPolygroupLayers = MeshIn->Attributes() ? MeshIn->Attributes()->NumPolygroupLayers() : 0;
	for (int32 li = 0; li < NumPolygroupLayers; ++li)
	{
		const FName LayerName = MeshIn->Attributes()->GetPolygroupLayer(li)->GetName();
		if (LayerName != NAME_None)
		{
			SourceLayerNames.Add(LayerName);
		}
	}

	// Drop stale non-reserved attributes
	{
		const TAttributesSet<FTriangleID>& ConstTriAttribsSet = TriAttribsSet;
		TArray<FName> AttributesToUnregister;
		ConstTriAttribsSet.ForEachByType<int32>([&AttributesToUnregister, &SourceLayerNames](const FName AttributeName, TTriangleAttributesConstRef<int32>)
		{
			if (!FSkeletalMeshAttributes::IsReservedAttributeName(AttributeName) && !SourceLayerNames.Contains(AttributeName))
			{
				AttributesToUnregister.Add(AttributeName);
			}
		});
		for (const FName AttributeName : AttributesToUnregister)
		{
			TriAttribsSet.UnregisterAttribute(AttributeName);
		}
	}

	TSet<FName> UniqueNames;
	int32 UniqueIDGenerator = 0;
	for (int32 li = 0; li < NumPolygroupLayers; ++li)
	{
		const FDynamicMeshPolygroupAttribute* Polygroups = MeshIn->Attributes()->GetPolygroupLayer(li);
		FName LayerName = Polygroups->GetName();
		while (LayerName == NAME_None || UniqueNames.Contains(LayerName))
		{
			FString BaseName = (Polygroups->GetName() == NAME_None) ? TEXT("Groups") : LayerName.ToString();
			LayerName = FName( FString::Printf(TEXT("%s_%d"), *BaseName, UniqueIDGenerator++) );
		}
		UniqueNames.Add(LayerName);	

		// Find existing attribute with the same name. If not found, create a new one.
		TTriangleAttributesRef<int32> Attribute;
		if (TriAttribsSet.HasAttribute(LayerName))
		{
			Attribute = TriAttribsSet.GetAttributesRef<int32>(LayerName);
		}
		else
		{
			TriAttribsSet.RegisterAttribute<int32>(LayerName, 1, 0, EMeshAttributeFlags::AutoGenerated);
			Attribute = TriAttribsSet.GetAttributesRef<int32>(LayerName);
		}
		if (ensure(Attribute.IsValid()))
		{
			for (int32 tid : MeshIn->TriangleIndicesItr())
			{
				FTriangleID TriangleID = IndexToTriangleIDMap[tid];
				int32 GroupID = Polygroups->GetValue(tid);
				Attribute.Set(TriangleID, GroupID);
			}
		}
		else
		{
			UE_LOGF(LogTemp, Warning, "FDynamicMeshToMeshDescription::ConvertPolygroupLayers - could not create attribute named %ls", *LayerName.ToString());
		}
	}
}


void FDynamicMeshToMeshDescription::ConvertWeightLayers(const FDynamicMesh3* MeshIn, FMeshDescription& MeshOut, const TArray<FVertexID>& IndexToVertexIDMap)
{
	TAttributesSet<FVertexID>& VertexAttribsSet = MeshOut.VertexAttributes();

	TSet<FName> SourceLayerNames;
	const int32 NumWeightLayers = MeshIn->Attributes() ? MeshIn->Attributes()->NumWeightLayers() : 0;
	for (int32 LayerIndex = 0; LayerIndex < NumWeightLayers; ++LayerIndex)
	{
		const FName LayerName = MeshIn->Attributes()->GetWeightLayer(LayerIndex)->GetName();
		if (LayerName != NAME_None)
		{
			SourceLayerNames.Add(LayerName);
		}
	}

	// Drop stale non-reserved attributes
	{
		const TAttributesSet<FVertexID>& ConstVertexAttribsSet = VertexAttribsSet;
		TArray<FName> AttributesToUnregister;
		ConstVertexAttribsSet.ForEachByType<float>([&AttributesToUnregister, &SourceLayerNames](const FName AttributeName, TVertexAttributesConstRef<float>)
		{
			if (!FSkeletalMeshAttributes::IsReservedAttributeName(AttributeName) && !SourceLayerNames.Contains(AttributeName))
			{
				AttributesToUnregister.Add(AttributeName);
			}
		});
		for (const FName AttributeName : AttributesToUnregister)
		{
			VertexAttribsSet.UnregisterAttribute(AttributeName);
		}
	}

	TSet<FName> UniqueNames;
	int32 UniqueIDGenerator = 0;
	for (int32 LayerIndex = 0; LayerIndex < NumWeightLayers; ++LayerIndex)
	{
		const FDynamicMeshWeightAttribute* Weights = MeshIn->Attributes()->GetWeightLayer(LayerIndex);
		FName LayerName = Weights->GetName();
		while (LayerName == NAME_None || UniqueNames.Contains(LayerName))
		{
			FString BaseName = (Weights->GetName() == NAME_None) ? TEXT("Weights") : LayerName.ToString();
			LayerName = FName( FString::Printf(TEXT("%s_%d"), *BaseName, UniqueIDGenerator++) );
		}
		UniqueNames.Add(LayerName);

		// Find existing attribute with the same name. If not found, create a new one.
		TVertexAttributesRef<float> Attribute;
		if (VertexAttribsSet.HasAttribute(LayerName))
		{
			Attribute = VertexAttribsSet.GetAttributesRef<float>(LayerName);
		}
		else
		{
			VertexAttribsSet.RegisterAttribute<float>(LayerName, 1, 0, EMeshAttributeFlags::AutoGenerated);
			Attribute = VertexAttribsSet.GetAttributesRef<float>(LayerName);
		}
		if (ensure(Attribute.IsValid()))
		{
			for (int32 InVertexID : MeshIn->VertexIndicesItr())
			{
				FVertexID OutVertexID = IndexToVertexIDMap[InVertexID];
				float Value;
				Weights->GetValue(InVertexID, &Value);
				Attribute.Set(OutVertexID, Value);
			}
		}
		else
		{
			UE_LOGF(LogTemp, Warning, "FDynamicMeshToMeshDescription::ConvertWeightLayers - could not create attribute named %ls", *LayerName.ToString());
		}
	}
}

namespace DynamicMeshToMeshDescriptionConversionHelper
{
	template <typename AttributeType>
	TTriangleAttributesRef<AttributeType> GetTriangleAttribute(TAttributesSet<FTriangleID>& TriAttribsSet, const FName AttrName, const AttributeType Default)
	{
		TTriangleAttributesRef<AttributeType> InvalidAttribute;
		
		if (TriAttribsSet.HasAttributeOfType<AttributeType>(AttrName))
		{
			TTriangleAttributesRef<AttributeType> ExistingAttribute = TriAttribsSet.GetAttributesRef<AttributeType>(AttrName);
			if (ExistingAttribute.GetNumChannels() == 1)
			{
				return ExistingAttribute;
			}
		}
		else if (TriAttribsSet.HasAttribute(AttrName))
		{ 
			return InvalidAttribute;	
		}
		else
		{
			TriAttribsSet.RegisterAttribute<AttributeType>(AttrName, 1, Default, EMeshAttributeFlags::AutoGenerated);
			return TriAttribsSet.GetAttributesRef<AttributeType>(AttrName);
		}

		return InvalidAttribute;
	}
}

void FDynamicMeshToMeshDescription::ConvertTriangleAttributes(const FDynamicMesh3* MeshIn, FMeshDescription& MeshOut, const TArray<FTriangleID>& IndexToTriangleIDMap)
{
	using namespace DynamicMeshToMeshDescriptionConversionHelper;

	TAttributesSet<FTriangleID>& MeshDescAttributes = MeshOut.TriangleAttributes();

	TSet<FName> SourceLayerNames;
	const FDynamicMeshAttributeSet* DynaMeshAttributes = MeshIn->Attributes();
	if (DynaMeshAttributes)
	{
		for (const auto& [AttrName, TriangleLabelAttr] : DynaMeshAttributes->GetTriangleLabelAttributes())
		{
			if (TriangleLabelAttr.IsValid())
			{
				SourceLayerNames.Add(AttrName);
			}
		}
	}

	// Drop stale non-reserved attributes
	{
		const TAttributesSet<FTriangleID>& ConstMeshDescAttributes = MeshDescAttributes;
		TArray<FName> AttributesToUnregister;
		ConstMeshDescAttributes.ForEachByType<FName>([&AttributesToUnregister, &SourceLayerNames](const FName AttributeName, TTriangleAttributesConstRef<FName>)
		{
			if (!FSkeletalMeshAttributes::IsReservedAttributeName(AttributeName) && !SourceLayerNames.Contains(AttributeName))
			{
				AttributesToUnregister.Add(AttributeName);
			}
		});
		for (const FName AttributeName : AttributesToUnregister)
		{
			MeshDescAttributes.UnregisterAttribute(AttributeName);
		}
	}

	if (!DynaMeshAttributes)
	{
		return;
	}

	for (const auto&[AttrName, TriangleLabelAttr]: DynaMeshAttributes->GetTriangleLabelAttributes())
	{
		if (TriangleLabelAttr.IsValid())
		{
			TTriangleAttributesRef<FName> Attribute = GetTriangleAttribute<FName>(MeshDescAttributes, AttrName, NAME_None);
			if (ensure(Attribute.IsValid()))
			{
				for (int32 tid : MeshIn->TriangleIndicesItr())
				{
					const FTriangleID TriangleID = IndexToTriangleIDMap[tid];
					Attribute.Set(TriangleID, TriangleLabelAttr->GetValue(tid));
				}
			}
			else
			{
				UE_LOGF(LogTemp, Warning, "FDynamicMeshToMeshDescription::ConvertTriangleAttributes - could not create attribute named %ls", *AttrName.ToString());
			}
		}
	}
}

void FDynamicMeshToMeshDescription::ApplyVertexColorTransform(FVector4f& Color) const
{
	// There is inconsistency in how vertex colors are intended to be consumed in
	// our shaders. Some shaders consume it as linear (ex. MeshPaint), others as SRGB which
	// manually convert to linear in the shader.
	//
	// All StaticMeshes store vertex colors as an 8-bit FColor. In order to ensure a good
	// distribution of float values across the 8-bit range, the StaticMesh build always
	// encodes FColors as SRGB.
	//
	// Until there is some defined gamma space convention for vertex colors in our shaders,
	// we provide this option to pre-transform our linear float colors with an SRGBToLinear
	// conversion to counteract the StaticMesh build LinearToSRGB conversion. This is how
	// MeshPaint ensures linear vertex colors in the shaders.
	if (ConversionOptions.bTransformVtxColorsSRGBToLinear)
	{
		LinearColors::SRGBToLinear(Color);
	}
}
