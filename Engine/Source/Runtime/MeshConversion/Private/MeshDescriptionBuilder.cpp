// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshDescriptionBuilder.h"
#include "StaticMeshAttributes.h"
#include "VectorTypes.h"
#include "BoxTypes.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

#include "DynamicMesh/MeshNormals.h"

using namespace UE::Geometry;

namespace ExtendedMeshAttribute
{
	const FName PolyTriGroups("PolyTriGroups");
}



void FMeshDescriptionBuilder::SetMeshDescription(FMeshDescription* Description)
{
	FStaticMeshAttributes Attributes(*Description);

	// make sure we have at least one UV channel
	if (!Attributes.GetVertexInstanceUVs().IsValid())
	{
		Description->VertexInstanceAttributes().RegisterAttribute<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate, 1, FVector2f::ZeroVector, EMeshAttributeFlags::Lerpable | EMeshAttributeFlags::Mandatory);
	}


	// handles to some of the standard attributes.
	this->MeshDescription = Description;
	this->VertexPositions = Attributes.GetVertexPositions();
	this->InstanceUVs = Attributes.GetVertexInstanceUVs();
	this->InstanceNormals = Attributes.GetVertexInstanceNormals();
	this->InstanceTangents = Attributes.GetVertexInstanceTangents();
	this->InstanceBiTangentSign = Attributes.GetVertexInstanceBinormalSigns();
	this->InstanceColors = Attributes.GetVertexInstanceColors();
	this->GroupMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();

}


void FMeshDescriptionBuilder::EnablePolyGroups()
{
	PolyGroups =
		MeshDescription->PolygonAttributes().GetAttributesRef<int>(ExtendedMeshAttribute::PolyTriGroups);
	if (PolyGroups.IsValid() == false)
	{
		MeshDescription->PolygonAttributes().RegisterAttribute<int>(
			ExtendedMeshAttribute::PolyTriGroups, 1, 0, EMeshAttributeFlags::AutoGenerated);
		PolyGroups =
			MeshDescription->PolygonAttributes().GetAttributesRef<int>(ExtendedMeshAttribute::PolyTriGroups);
		check(PolyGroups.IsValid());
	}
}




void FMeshDescriptionBuilder::ReserveNewVertices(int32 Count)
{
	MeshDescription->ReserveNewVertices(Count);
}




FVertexID FMeshDescriptionBuilder::AppendVertex(const FVector& Position)
{
	FVertexID VertexID = MeshDescription->CreateVertex();
	VertexPositions.Set(VertexID, FVector3f(Position));	//LWC_TODO: Precision loss
	return VertexID;
}

FVertexID FMeshDescriptionBuilder::AppendVertexWithId(int32 NewVertexID, const FVector& Position)
{
	FVertexID VertexID{NewVertexID};
	MeshDescription->CreateVertexWithID(VertexID);
	VertexPositions.Set(VertexID, FVector3f(Position));	//LWC_TODO: Precision loss
	return VertexID;
}


FPolygonGroupID FMeshDescriptionBuilder::AppendPolygonGroup(FName MaterialSlotName)
{
	FPolygonGroupID NewPolygonGroupID = MeshDescription->CreatePolygonGroup();
	GroupMaterialSlotNames.Set(NewPolygonGroupID, MaterialSlotName);
	return NewPolygonGroupID;
}



FTriangleID FMeshDescriptionBuilder::AppendTriangle(const FVertexID& Vertex0, const FVertexID& Vertex1, const FVertexID& Vertex2, const FPolygonGroupID& PolygonGroup)
{
	
	FVertexInstanceID TriVertexInstances[3];
	TriVertexInstances[0] = MeshDescription->CreateVertexInstance(Vertex0);
	TriVertexInstances[1] = MeshDescription->CreateVertexInstance(Vertex1);
	TriVertexInstances[2] = MeshDescription->CreateVertexInstance(Vertex2);


	return AppendTriangle(TriVertexInstances[0], TriVertexInstances[1], TriVertexInstances[2], PolygonGroup);
}


FVertexInstanceID FMeshDescriptionBuilder::AppendInstance(const FVertexID& VertexID)
{
	return MeshDescription->CreateVertexInstance(VertexID);
}

void FMeshDescriptionBuilder::ReserveNewUVs(int32 Count, int UVLayerIndex)
{
	// the reserve new UVs can only be called after the UV channels have been created.
	
	check(MeshDescription->GetNumUVElementChannels() > UVLayerIndex )

	MeshDescription->ReserveNewUVs(Count, UVLayerIndex);
}

FUVID FMeshDescriptionBuilder::AppendUV(const FVector2D& UVvalue, int32 UVLayerIndex)
{
	TUVAttributesRef<FVector2f> UVCoordinates = UVCoordinateLayers[UVLayerIndex];
	FUVID UVID = MeshDescription->CreateUV(UVLayerIndex);
	UVCoordinates[UVID] = FVector2f(UVvalue);
	return UVID;
}

void FMeshDescriptionBuilder::AppendUVTriangle(const FTriangleID& TriangleID, const FUVID UVvertexID0, const FUVID UVvertexID1, const FUVID UVvertexID2, int32 UVLayerIndex)
{
	// set the shared UVs
	TempUVBuffer.SetNum(3, EAllowShrinking::No);
	TempUVBuffer[0] = UVvertexID0;
	TempUVBuffer[1] = UVvertexID1;
	TempUVBuffer[2] = UVvertexID2;
	MeshDescription->SetTriangleUVIndices(TriangleID, TempUVBuffer, UVLayerIndex);

	TUVAttributesRef<FVector2f> UVCoordinates = UVCoordinateLayers[UVLayerIndex];

	// set per-instance UVs.  
	// NB: per-instance UVs should go away on MeshDescription.
	TArrayView<const FVertexInstanceID> TriVertInstances = MeshDescription->GetTriangleVertexInstances(TriangleID);
	for (int32 j = 0; j < 3; ++j)
	{
		const FVertexInstanceID CornerInstanceID = TriVertInstances[j];
		const FVector2f& UVvalue = UVCoordinates[TempUVBuffer[j]];
		
		SetInstanceUV(CornerInstanceID, FVector2D(UVvalue), UVLayerIndex);
	}
}

void FMeshDescriptionBuilder::SetPosition(const FVertexID& VertexID, const FVector& NewPosition)
{
	VertexPositions.Set(VertexID, 0, FVector3f(NewPosition));	//LWC_TODO: Precision loss
}

FVector FMeshDescriptionBuilder::GetPosition(const FVertexID& VertexID)
{
	return FVector(VertexPositions.Get(VertexID, 0));
}

FVector FMeshDescriptionBuilder::GetPosition(const FVertexInstanceID& InstanceID)
{
	return FVector(VertexPositions.Get(MeshDescription->GetVertexInstanceVertex(InstanceID), 0));
}



void FMeshDescriptionBuilder::SetInstanceNormal(const FVertexInstanceID& InstanceID, const FVector& Normal)
{
	if (InstanceNormals.IsValid())
	{
		InstanceNormals.Set(InstanceID, FVector3f(Normal));	//LWC_TODO: Precision loss
	}
}

void FMeshDescriptionBuilder::SetInstanceTangentSpace(const FVertexInstanceID& InstanceID, const FVector& Normal, const FVector& Tangent, float Sign)
{
	// set the normal
	SetInstanceNormal(InstanceID, Normal);

	if (InstanceTangents.IsValid())
	{
		InstanceTangents.Set(InstanceID, FVector3f(Tangent));	//LWC_TODO: Precision loss
	}
	if (InstanceBiTangentSign.IsValid())
	{
		InstanceBiTangentSign.Set(InstanceID, Sign);
	}


}


void FMeshDescriptionBuilder::SetInstanceUV(const FVertexInstanceID& InstanceID, const FVector2D& InstanceUV, int32 UVLayerIndex)
{
	if (InstanceUVs.IsValid() && ensure(UVLayerIndex < InstanceUVs.GetNumChannels()))
	{
		InstanceUVs.Set(InstanceID, UVLayerIndex, FVector2f(InstanceUV));
	}
}


void FMeshDescriptionBuilder::SetNumUVLayers(int32 NumUVLayers)
{
	bool bValidInstanceUVs = InstanceUVs.IsValid();
	if (bValidInstanceUVs)
	{
		// initialize the instanced UV channels
		InstanceUVs.SetNumChannels(NumUVLayers);
	}

	// initialize the shared UV channels
	MeshDescription->SetNumUVChannels(NumUVLayers);

	// cache reference to uv vertex buffers
	UVCoordinateLayers.SetNum(NumUVLayers);
	for (int i = 0; i < NumUVLayers; ++i)
	{
		UVCoordinateLayers[i] = MeshDescription->UVAttributes(i).GetAttributesRef<FVector2f>(MeshAttribute::UV::UVCoordinate);
	}
}


void FMeshDescriptionBuilder::SetInstanceColor(const FVertexInstanceID& InstanceID, const FVector4f& Color)
{
	if (InstanceColors.IsValid())
	{
		InstanceColors.Set(InstanceID, Color);
	}
}


FTriangleID FMeshDescriptionBuilder::AppendTriangle(const FVertexID* Triangle, const FPolygonGroupID& PolygonGroup)
{
	FVertexInstanceID TriVertexInstances[3];
	TriVertexInstances[0] = MeshDescription->CreateVertexInstance(Triangle[0]);
	TriVertexInstances[1] = MeshDescription->CreateVertexInstance(Triangle[1]);
	TriVertexInstances[2] = MeshDescription->CreateVertexInstance(Triangle[2]);

	return AppendTriangle(TriVertexInstances[0], TriVertexInstances[1], TriVertexInstances[2], PolygonGroup);
}


FPolygonID FMeshDescriptionBuilder::AppendPolygon(const TArray<FVertexID>& Vertices, const FPolygonGroupID& PolygonGroup)
{
	int NumVertices = Vertices.Num();
	TArray<FVertexInstanceID> Polygon; 
	Polygon.Reserve(NumVertices);
	for (int j = 0; j < NumVertices; ++j) 
	{
		FVertexInstanceID VertexInstance = MeshDescription->CreateVertexInstance(Vertices[j]);
		Polygon.Add(VertexInstance);

	}

	const FPolygonID NewPolygonID = MeshDescription->CreatePolygon(PolygonGroup, Polygon);

	return NewPolygonID;
}




FTriangleID FMeshDescriptionBuilder::AppendTriangle(const FVertexInstanceID& Instance0, const FVertexInstanceID& Instance1, const FVertexInstanceID& Instance2, const FPolygonGroupID& PolygonGroup)
{
	TArray<FVertexInstanceID> CornerInstanceIDs;
	CornerInstanceIDs.Add(Instance0);
	CornerInstanceIDs.Add(Instance1);
	CornerInstanceIDs.Add(Instance2);

	const FTriangleID NewTriangleID = MeshDescription->CreateTriangle(PolygonGroup, CornerInstanceIDs);

	return NewTriangleID;
}





void FMeshDescriptionBuilder::SetPolyGroupID(const FTriangleID& TriangleID, int GroupID)
{
	FPolygonID PolygonID = MeshDescription->GetTrianglePolygon(TriangleID);
	PolyGroups.Set(PolygonID, 0, GroupID);
}









void FMeshDescriptionBuilder::Translate(const FVector& Translation)
{
	for (FVertexID VertexID : MeshDescription->Vertices().GetElementIDs())
	{
		FVector3f Position = VertexPositions.Get(VertexID);
		Position += FVector3f(Translation);		//LWC_TODO: Precision loss
		VertexPositions.Set(VertexID, Position);
	}
}




void FMeshDescriptionBuilder::SetAllEdgesHardness(bool bHard)
{
	TEdgeAttributesRef<bool> EdgeHardness =
		MeshDescription->EdgeAttributes().GetAttributesRef<bool>(MeshAttribute::Edge::IsHard);
	for (FEdgeID EdgeID : MeshDescription->Edges().GetElementIDs())
	{
		EdgeHardness.Set(EdgeID, 0, bHard);
	}
}




FBox FMeshDescriptionBuilder::ComputeBoundingBox() const
{
	FAxisAlignedBox3f bounds = FAxisAlignedBox3f::Empty();
	for ( FVertexID VertexID : MeshDescription->Vertices().GetElementIDs() )
	{
		bounds.Contain((FVector3f)VertexPositions.Get(VertexID));
	}
	return (FBox)bounds;
}




void FMeshDescriptionBuilder::SuspendMeshDescriptionIndexing()
{
	MeshDescription->SuspendVertexInstanceIndexing();
	MeshDescription->SuspendEdgeIndexing();
	MeshDescription->SuspendPolygonIndexing();
	MeshDescription->SuspendPolygonGroupIndexing();
	MeshDescription->SuspendUVIndexing();
}



void FMeshDescriptionBuilder::ResumeMeshDescriptionIndexing()
{
	MeshDescription->ResumeVertexInstanceIndexing();
	MeshDescription->ResumeEdgeIndexing();
	MeshDescription->ResumePolygonIndexing();
	MeshDescription->ResumePolygonGroupIndexing();
	MeshDescription->ResumeUVIndexing();
}