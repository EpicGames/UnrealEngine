// Copyright Epic Games, Inc. All Rights Reserved. 

#include "MeshDescriptionToDynamicMesh.h"

#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMeshOverlay.h"
#include "DynamicMesh/DynamicVertexAttribute.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "DynamicMesh/DynamicBoneAttribute.h"
#include "DynamicMesh/MeshTangents.h"
#include "DynamicMesh/NonManifoldMappingSupport.h"
#include "MeshDescriptionBuilder.h"
#include "StaticMeshAttributes.h"
#include "SkeletalMeshAttributes.h"
#include "Tasks/Task.h"
#include "Util/ColorConstants.h"

using namespace UE::Geometry;

struct FVertexUV
{
	int vid;
	float x;
	float y;
	bool operator==(const FVertexUV & o) const
	{
		return vid == o.vid && x == o.x && y == o.y;
	}
};
FORCEINLINE uint32 GetTypeHash(const FVertexUV& Vector)
{
	// ugh copied from FVector clearly should not be using CRC for hash!!
	return FCrc::MemCrc32(&Vector, sizeof(Vector));
}


class FUVWelder
{
public:
	TMap<FVertexUV, int> UniqueVertexUVs;
	FDynamicMeshUVOverlay* UVOverlay;

	FUVWelder() : UVOverlay(nullptr)
	{
	}

	FUVWelder(FDynamicMeshUVOverlay* UVOverlayIn)
	{
		check(UVOverlayIn);
		UVOverlay = UVOverlayIn;
	}

	int FindOrAddUnique(const FVector2f& UV, int VertexID)
	{
		FVertexUV VertUV = { VertexID, UV.X, UV.Y };

		const int32* FoundIndex = UniqueVertexUVs.Find(VertUV);
		if (FoundIndex != nullptr)
		{
			return *FoundIndex;
		}

		int32 NewIndex = UVOverlay->AppendElement(FVector2f(UV));
		UniqueVertexUVs.Add(VertUV, NewIndex);
		return NewIndex;
	}
};




struct FVertexNormal
{
	int vid;
	float x;
	float y;
	float z;
	bool operator==(const FVertexNormal & o) const
	{
		return vid == o.vid && x == o.x && y == o.y && z == o.z;
	}
};
FORCEINLINE uint32 GetTypeHash(const FVertexNormal& Vector)
{
	// ugh copied from FVector clearly should not be using CRC for hash!!
	return FCrc::MemCrc32(&Vector, sizeof(Vector));
}


class FNormalWelder
{
public:
	TMap<FVertexNormal, int> UniqueVertexNormals;
	FDynamicMeshNormalOverlay* NormalOverlay;

	FNormalWelder() : NormalOverlay(nullptr)
	{
	}

	FNormalWelder(FDynamicMeshNormalOverlay* NormalOverlayIn)
	{
		check(NormalOverlayIn);
		NormalOverlay = NormalOverlayIn;
	}

	int FindOrAddUnique(const FVector3f & Normal, int VertexID)
	{
		FVertexNormal VertNormal = { VertexID, Normal.X, Normal.Y, Normal.Z };

		const int32* FoundIndex = UniqueVertexNormals.Find(VertNormal);
		if (FoundIndex != nullptr)
		{
			return *FoundIndex;
		}

		int32 NewIndex = NormalOverlay->AppendElement( &Normal.X );
		UniqueVertexNormals.Add(VertNormal, NewIndex);
		return NewIndex;
	}
};

struct FVertexColor
{
	int vid;
	FVector4f Color;
	bool operator==(const FVertexColor& o) const
	{
		return vid == o.vid && Color == o.Color;
	}
};
FORCEINLINE uint32 GetTypeHash(const FVertexColor& Vector)
{
	return FCrc::MemCrc32(&Vector, sizeof(Vector));
}

class FColorWelder
{
public:
	TMap<FVertexColor, int> UniqueVertexColors;
	FDynamicMeshColorOverlay* ColorOverlay;

	FColorWelder() : ColorOverlay(nullptr)
	{
	}

	FColorWelder(FDynamicMeshColorOverlay* ColorOverlayIn)
	{
		check(ColorOverlayIn);
		ColorOverlay = ColorOverlayIn;
	}
	int FindOrAddUnique(const FVector4f& Color, int VertexID)
	{
		FVertexColor VertColor = { VertexID, Color };

		const int32* FoundIndex = UniqueVertexColors.Find(VertColor);
		if (FoundIndex != nullptr)
		{
			return *FoundIndex;
		}

		int32 NewIndex = ColorOverlay->AppendElement(&Color.X);
		UniqueVertexColors.Add(VertColor, NewIndex);
		return NewIndex;
	}
};


struct FSkinWeightsAttribCopyInfo
{
	FSkinWeightsVertexAttributesConstRef MeshDescAttribRef;
	FDynamicMeshVertexSkinWeightsAttribute* DynaMeshAttribRef = nullptr;
};

struct FMorphTargetAttribCopyInfo
{
	TVertexAttributesConstRef<FVector3f> MeshDescAttribRef;
	FDynamicMeshMorphTargetAttribute* DynaMeshAttribRef = nullptr;
};

struct FTriangleLabelAttribCopyInfo
{
	TTriangleAttributesConstRef<FName> MeshDescAttribRef;
	FDynamicMeshTriangleLabelAttribute* DynaMeshAttribRef = nullptr;
};

void FMeshDescriptionToDynamicMesh::Convert(const FMeshDescription* MeshIn, FDynamicMesh3& MeshOut, bool bCopyTangents )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshDescriptionToDynamicMesh);

	TriIDMap.Reset();
	VertIDMap.Reset();
	MeshOut.Clear();

	if (!ensure(MeshIn != nullptr))
	{
		return; // nothing to convert
	}

	// allocate the VertIDMap.  Unfortunately the array will need to grow more if MeshIn has non-manifold edges that need to be split
	VertIDMap.SetNumUninitialized(MeshIn->Vertices().Num());

	// look up vertex positions
	TVertexAttributesConstRef<FVector3f> VertexPositions = MeshIn->GetVertexPositions();

	// copy vertex positions. Later we may have to append duplicate vertices to resolve non-manifold structures.
	for (const FVertexID VertexID : MeshIn->Vertices().GetElementIDs())
	{
		const FVector3f Position = VertexPositions.Get(VertexID);
		int NewVertIdx = MeshOut.AppendVertex( (FVector3d)Position);
		VertIDMap[NewVertIdx] = VertexID;
	}

	// look up vertex-instance UVs and normals
	// @todo: does the MeshDescription always have UVs and Normals?
	FSkeletalMeshConstAttributes Attributes(*MeshIn);
	TVertexInstanceAttributesConstRef<FVector2f> InstanceUVs = Attributes.GetVertexInstanceUVs();
	TVertexInstanceAttributesConstRef<FVector3f> InstanceNormals = Attributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesConstRef<FVector3f> InstanceTangents = Attributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesConstRef<float> InstanceBiTangentSign = Attributes.GetVertexInstanceBinormalSigns();

	TVertexInstanceAttributesConstRef<FVector4f> InstanceColors = Attributes.GetVertexInstanceColors();


	TPolygonAttributesConstRef<int> PolyGroups =
		MeshIn->PolygonAttributes().GetAttributesRef<int>(ExtendedMeshAttribute::PolyTriGroups);
	if (bEnableOutputGroups)
	{
		MeshOut.EnableTriangleGroups(0);
	}

	// If needed, compute mapping to 'compacted' versions of the Polygon Group IDs
	TArray<int32> CompactedPolygonGroupMap;
	if (bUseCompactedPolygonGroupIDValues && MeshIn->PolygonGroups().Num() != MeshIn->PolygonGroups().GetArraySize())
	{
		const FPolygonGroupArray& PolygonGroups = MeshIn->PolygonGroups();
		CompactedPolygonGroupMap.Init(INDEX_NONE, PolygonGroups.GetArraySize());
		int32 SectionIndex = 0;
		for (FPolygonGroupID GroupID : PolygonGroups.GetElementIDs())
		{
			CompactedPolygonGroupMap[GroupID.GetValue()] = SectionIndex;
			++SectionIndex;
		}
	}
	auto MapPolygonGroupID = [&CompactedPolygonGroupMap](FPolygonGroupID GroupID) -> int32
	{
		return CompactedPolygonGroupMap.IsEmpty() ? GroupID.GetValue() : CompactedPolygonGroupMap[GroupID.GetValue()];
	};

	bCopyTangents = bCopyTangents && InstanceTangents.IsValid() && InstanceBiTangentSign.IsValid();
	
	// base triangle groups will track polygons
	int NumVertices = MeshIn->Vertices().Num();
	int NumPolygons = MeshIn->Polygons().Num();
	int NumVtxInstances = MeshIn->VertexInstances().Num();
	if (bPrintDebugMessages)
	{
		UE_LOGF(LogTemp, Warning, "FMeshDescriptionToDynamicMesh: MeshDescription verts %d polys %d instances %d", NumVertices, NumPolygons, NumVtxInstances);
	}

	FDateTime Time_AfterVertices = FDateTime::Now();

	// Although it is slightly redundant, we will build up this list of MeshDescription data
	// so that it is easier to index into it below (profile whether extra memory here hurts us?)
	struct FTriData
	{
		FPolygonID PolygonID;
		int32 PolygonGroupID;
		FVertexInstanceID TriInstances[3];
	};
	TArray<FTriData> AddedTriangles;
	AddedTriangles.SetNum(MeshIn->Triangles().Num());

	// allocate the TriIDMap (maps from DynamicMesh triangle ID to MeshDescription FTriangleID)
	TriIDMap.AddUninitialized(MeshIn->Triangles().Num());
	
	bool bSrcIsManifold = true;

	// Iterate over triangles in the Mesh Description
	// NOTE: If you change the iteration order here, please update the corresponding iteration in FDynamicMeshToMeshDescription::UpdateAttributes, 
	//	which assumes the iteration order here is the same as here to correspond the triangles when writing updated attributes back!
	for (const FTriangleID TriangleID : MeshIn->Triangles().GetElementIDs())
	{

		// Get the PolygonID, PolygonGroupID and general GroupID
		//---
		FPolygonID PolygonID = MeshIn->GetTrianglePolygon(TriangleID);
		int32 MappedPolygonGroupID = MapPolygonGroupID(MeshIn->GetTrianglePolygonGroup(TriangleID));

		int GroupID = 0;
		switch (GroupMode)
		{
			case EPrimaryGroupMode::SetToPolyGroup:
				if (PolyGroups.IsValid())
				{
					GroupID = PolyGroups.Get(PolygonID, 0);
				}
				break;
			case EPrimaryGroupMode::SetToPolygonID:
				GroupID = PolygonID.GetValue() + 1; // Shift IDs up by 1 to leave ID 0 as a default/unassigned group
				break;
			case EPrimaryGroupMode::SetToPolygonGroupID:
				GroupID = MappedPolygonGroupID + 1; // Shift IDs up by 1 to leave ID 0 as a default/unassigned group
				break;
			case EPrimaryGroupMode::SetToZero:
				break; // keep at 0
		}
		
		FTriData TriData;
		TriData.PolygonID = PolygonID;
		TriData.PolygonGroupID = MappedPolygonGroupID;
		

		// stash the vertex instance IDs for this triangle.  potentially needed for per-instance attribute welding
		TArrayView<const FVertexInstanceID> InstanceTri = MeshIn->GetTriangleVertexInstances(TriangleID);
		TriData.TriInstances[0] = InstanceTri[0];
		TriData.TriInstances[1] = InstanceTri[1];
		TriData.TriInstances[2] = InstanceTri[2];

		//---

		// Get the vertex IDs for this triangle.
		TArrayView<const FVertexID> TriangleVertexIDs = MeshIn->GetTriangleVertices(TriangleID);

		// sanity
		checkSlow(TriangleVertexIDs.Num() == 3);

		FIndex3i VertexIDs; 
		VertexIDs[0] = TriangleVertexIDs[0].GetValue();
		VertexIDs[1] = TriangleVertexIDs[1].GetValue(); 
		VertexIDs[2] = TriangleVertexIDs[2].GetValue();

		int NewTriangleID = MeshOut.AppendTriangle(VertexIDs, GroupID);

		// Deal with potential failure cases

		//-- already seen this triangle for some reason.. or the MeshDecription had a degenerate tri
		if (NewTriangleID == FDynamicMesh3::DuplicateTriangleID || NewTriangleID == FDynamicMesh3::InvalidID)
		{
			continue;
		}

		//-- non manifold 
		// if append failed due to non-manifold, duplicate verts
		if (NewTriangleID == FDynamicMesh3::NonManifoldID)
		{
			bSrcIsManifold = false;

			int e0 = MeshOut.FindEdge(VertexIDs[0], VertexIDs[1]);
			int e1 = MeshOut.FindEdge(VertexIDs[1], VertexIDs[2]);
			int e2 = MeshOut.FindEdge(VertexIDs[2], VertexIDs[0]);

			// determine which verts need to be duplicated
			bool bDuplicate[3] = { false, false, false };
			if (e0 != FDynamicMesh3::InvalidID && MeshOut.IsBoundaryEdge(e0) == false)
			{
				bDuplicate[0] = true;
				bDuplicate[1] = true;
			}
			if (e1 != FDynamicMesh3::InvalidID && MeshOut.IsBoundaryEdge(e1) == false)
			{
				bDuplicate[1] = true;
				bDuplicate[2] = true;
			}
			if (e2 != FDynamicMesh3::InvalidID && MeshOut.IsBoundaryEdge(e2) == false)
			{
				bDuplicate[2] = true;
				bDuplicate[0] = true;
			}
			for (int32 i = 0; i < 3; ++i)
			{
				if (bDuplicate[i])
				{
					const FVertexID& TriangleVertID = TriangleVertexIDs[i];
					const FVector3f Position = VertexPositions[TriangleVertID];
					const int32 NewVertIdx = MeshOut.AppendVertex( (FVector3d)Position );
					VertexIDs[i] = NewVertIdx;
					VertIDMap.SetNumUninitialized(NewVertIdx + 1);
					VertIDMap[NewVertIdx] = TriangleVertID;
				}
			}
			

			NewTriangleID = MeshOut.AppendTriangle(VertexIDs, GroupID);
			checkSlow(NewTriangleID != FDynamicMesh3::NonManifoldID);
		}

		checkSlow(NewTriangleID >= 0);
		AddedTriangles[NewTriangleID] = TriData;
		
		TriIDMap[NewTriangleID] = TriangleID;
	
		}

	int32 MaxTriID = MeshOut.MaxTriangleID(); // really MaxTriID+1
	// if the source mesh had duplicates then TriIDMap will be too long, this will trim excess
	TriIDMap.SetNum(MaxTriID);
	


	FDateTime Time_AfterTriangles = FDateTime::Now();


	//
	// Enable relevant attributes and initialize UV/Normal welders
	// 

	// the shared UV representation.  
	const int32 NumUVElementChannels = MeshIn->GetNumUVElementChannels();

	// the instanced UV representation.
	const int NumUVLayers = InstanceUVs.IsValid() ? InstanceUVs.GetNumChannels() : 0;

	// determine if we really have shared UVS. Legacy geo might not have them 
	// - at the time of this writing MeshDescription has not been updated
	// to always populated the shared UVs during load time for legacy geometry that only has per-instance UVs. 
	// but that is a promised improvement. 

	bool bUseSharedUVs =  (NumUVLayers == NumUVElementChannels);
	if (bUseSharedUVs)
	{
		for (int UVLayerIndex = 0; UVLayerIndex < NumUVLayers; ++UVLayerIndex)
		{
			const int32 NumSharedUVs   = MeshIn->UVs(UVLayerIndex).GetArraySize();
			bUseSharedUVs = bUseSharedUVs && (NumSharedUVs != 0);
		}
	}

	FDynamicMeshColorOverlay* ColorOverlay = nullptr;
	FDynamicMeshNormalOverlay* NormalOverlay = nullptr;
	FDynamicMeshNormalOverlay* TangentOverlay = nullptr;
	FDynamicMeshNormalOverlay* BiTangentOverlay = nullptr;
	FDynamicMeshMaterialAttribute* MaterialIDAttrib = nullptr;
	FDynamicMeshBoneNameAttribute* BoneNameAttrib = nullptr;
	FDynamicMeshBoneParentIndexAttribute* BoneParentIndexAttrib = nullptr;
	FDynamicMeshBonePoseAttribute* BonePoseAttrib = nullptr;
	FDynamicMeshBoneColorAttribute* BoneColorAttrib = nullptr;

	TArray<FSkinWeightsAttribCopyInfo> SkinWeightAttribs;
	TArray<FMorphTargetAttribCopyInfo> MorphTargetAttribs;
	TArray<FTriangleLabelAttribCopyInfo> TriangleLabelAttributes;
	
	if (!bDisableAttributes)
	{
		MeshOut.EnableAttributes(); // by default 1-UV layer and 1-normal layer

		// Normals
		// set up for the tangent plane vectors if required
		int32 NumRequiredNormalLayers = (bCopyTangents) ? 3 : 1;
		if (MeshOut.Attributes()->NumNormalLayers() <  NumRequiredNormalLayers)
		{	
			// add additional normal layers and reserve space in them
			MeshOut.Attributes()->SetNumNormalLayers(NumRequiredNormalLayers);
		}
		for (int32 i = 0; i < NumRequiredNormalLayers; ++i)
		{ 
			MeshOut.Attributes()->GetNormalLayer(i)->InitializeTriangles(MeshOut.MaxTriangleID());
		}

		NormalOverlay = MeshOut.Attributes()->PrimaryNormals();
		if (bCopyTangents)
		{ 
			TangentOverlay = MeshOut.Attributes()->PrimaryTangents();
			BiTangentOverlay = MeshOut.Attributes()->PrimaryBiTangents();
		}
		
		// UVs 
		MeshOut.Attributes()->SetNumUVLayers(NumUVLayers);
		// reserve space in any new UV layers.
		for (int32 i = 1; i < NumUVLayers; ++i)
		{
			MeshOut.Attributes()->GetUVLayer(i)->InitializeTriangles(MeshOut.MaxTriangleID());
		}
	
		if (InstanceColors.IsValid())
		{
			MeshOut.Attributes()->EnablePrimaryColors();
			ColorOverlay = MeshOut.Attributes()->PrimaryColors();
		}


		// always enable Material ID if there are any attributes
		MeshOut.Attributes()->EnableMaterialID();
		MaterialIDAttrib = MeshOut.Attributes()->GetMaterialID();

		// Copy all skin weight attributes, if they exist.
		MeshIn->VertexAttributes().ForEach([&](const FName InAttributeName, auto InAttributesRef)
		{
			if (FSkeletalMeshAttributes::IsSkinWeightAttribute(InAttributeName))
			{
				const FName ProfileName = FSkeletalMeshAttributes::GetProfileNameFromAttribute(InAttributeName);
				
                FDynamicMeshVertexSkinWeightsAttribute *VertexSkinWeightsAttrib = new FDynamicMeshVertexSkinWeightsAttribute(&MeshOut);

                SkinWeightAttribs.Add({
                    Attributes.GetVertexSkinWeightsFromAttributeName(InAttributeName),
                	VertexSkinWeightsAttrib                    
                });
				MeshOut.Attributes()->AttachSkinWeightsAttribute(ProfileName, VertexSkinWeightsAttrib);
			}
		});

		// Copy all morph targets
		MeshIn->VertexAttributes().ForEach([&](const FName InAttributeName, auto InAttributesRef)
		{
			if (FSkeletalMeshAttributes::IsMorphTargetAttribute(InAttributeName))
			{
				const FName MorphTargetName = FSkeletalMeshAttributes::GetMorphTargetNameFromAttribute(InAttributeName);
				
				FDynamicMeshMorphTargetAttribute *MorphTargetAttribute = new FDynamicMeshMorphTargetAttribute(&MeshOut);
				MorphTargetAttribs.Add({
					Attributes.GetVertexMorphPositionDelta(MorphTargetName),
					MorphTargetAttribute                    
				});
				
				MeshOut.Attributes()->AttachMorphTargetAttribute(MorphTargetName, MorphTargetAttribute);
			}
		});
		
		if (Attributes.HasBones())
		{	
			MeshOut.Attributes()->EnableBones(Attributes.GetNumBones());
			if (Attributes.HasBoneNameAttribute())
			{
				BoneNameAttrib = MeshOut.Attributes()->GetBoneNames();
			}
			
			if (Attributes.HasBoneParentIndexAttribute())
			{
				BoneParentIndexAttrib = MeshOut.Attributes()->GetBoneParentIndices();
			}

			if (Attributes.HasBonePoseAttribute())
			{
				BonePoseAttrib = MeshOut.Attributes()->GetBonePoses();
			}

			if (Attributes.HasBoneColorAttribute())
			{
				BoneColorAttrib = MeshOut.Attributes()->GetBoneColors();
			}
		}
		
		// find triangle label attribs
		const TAttributesSet<FTriangleID>& TriAttribsSet = MeshIn->TriangleAttributes();
		TriAttribsSet.ForEach([&TriAttribsSet, &TriangleLabelAttributes, &MeshOut](const FName AttributeName, auto AttributesRef)
		{
			if (FSkeletalMeshAttributes::IsReservedAttributeName(AttributeName)) return;

			if (TriAttribsSet.template HasAttributeOfType<FName>(AttributeName))
			{
				FDynamicMeshTriangleLabelAttribute* NewTriangleLabelAttrib = new FDynamicMeshTriangleLabelAttribute(&MeshOut);
				MeshOut.Attributes()->AttachTriangleLabelAttribute(AttributeName, NewTriangleLabelAttrib);
				TriangleLabelAttributes.Add({TriAttribsSet.GetAttributesRef<FName>(AttributeName), NewTriangleLabelAttrib});
			}
		});
	}


	// find polygroup attribs
	const TAttributesSet<FTriangleID>& TriAttribsSet = MeshIn->TriangleAttributes();
	TArray<TTriangleAttributesConstRef<int32>> PolygroupAttribs;
	TArray<FName> PolygroupAttribNames;
	TriAttribsSet.ForEach([&](const FName AttributeName, auto AttributesRef)
	{
		if (FSkeletalMeshAttributes::IsReservedAttributeName(AttributeName)) return;

		if (TriAttribsSet.template HasAttributeOfType<int32>(AttributeName))
		{
			PolygroupAttribs.Add(TriAttribsSet.GetAttributesRef<int32>(AttributeName));
			PolygroupAttribNames.Add(AttributeName);
		}
	});


	// find weight attribs
	const TAttributesSet<FVertexID>& VertAttribsSet = MeshIn->VertexAttributes();
	TArray<TVertexAttributesConstRef<float>> WeightAttribs;
	TArray<FName> WeightAttribNames;
	VertAttribsSet.ForEach([&](const FName AttributeName, auto AttributesRef)
	{
		if (FSkeletalMeshAttributes::IsReservedAttributeName(AttributeName)) return;
		if (FSkeletalMeshAttributes::IsSkinWeightAttribute(AttributeName)) return;

		if (VertAttribsSet.template HasAttributeOfType<float>(AttributeName))
		{
			WeightAttribs.Add(VertAttribsSet.GetAttributesRef<float>(AttributeName));
			WeightAttribNames.Add(AttributeName);
		}
	});


	if (!bDisableAttributes)
	{
		bool bFoundNonDefaultVertexInstanceColor = false;

		// we will weld/populate all the attributes simultaneously, hold on to futures in this array and then Wait for them at the end
		TArray<UE::Tasks::FTask> Pending;

		if (PolygroupAttribs.Num() > 0)
		{
			MeshOut.Attributes()->SetNumPolygroupLayers(PolygroupAttribs.Num());
		}

		MeshOut.Attributes()->SetNumWeightLayers(WeightAttribs.Num());

		for (int UVLayerIndex = 0; UVLayerIndex < NumUVLayers; UVLayerIndex++)
		{
			UE::Tasks::FTask UVTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&, UVLayerIndex, bUseSharedUVs]() // must copy UVLayerIndex here!
			{
				// the overlay to fill.
				FDynamicMeshUVOverlay* UVOverlay = MeshOut.Attributes()->GetUVLayer(UVLayerIndex);
				if (UVOverlay == nullptr) return;

				if (!bUseSharedUVs) // have to rely on welding the per-instance uvs 
				{
					
					FUVWelder UVWelder;
					UVWelder.UVOverlay = UVOverlay;

					for (int32 TriangleID : MeshOut.TriangleIndicesItr())
					{
						FIndex3i Tri = MeshOut.GetTriangle(TriangleID);
						const FTriData& TriData = AddedTriangles[TriangleID];
						FIndex3i TriUV;
						for (int j = 0; j < 3; ++j)
						{
							FVector2f UV = InstanceUVs.Get(TriData.TriInstances[j], UVLayerIndex);
							TriUV[j] = UVWelder.FindOrAddUnique(UV, Tri[j]);
						}
						UVOverlay->SetTriangle(TriangleID, TriUV);
					}
				}
				else
				{

					// copy uv "vertex buffer"
					const FUVArray& UVs = MeshIn->UVs(UVLayerIndex);
					TUVAttributesRef<const FVector2f> UVCoordinates = UVs.GetAttributes().GetAttributesRef<FVector2f>(MeshAttribute::UV::UVCoordinate);
						
					// map to translate UVIds from FUVID::int32() to DynamicOverlay Index.
					TArray<int32> UVIndexMap;
					{
						// pre-compute max UVID in the mesh description
						int32 MaxUVElID = -1;
						for (FUVID UVID : UVs.GetElementIDs())
						{
							MaxUVElID = FMath::Max(MaxUVElID, UVID.GetValue());
						}
						UVIndexMap.AddUninitialized(MaxUVElID+1);
					}
					for (FUVID UVID : UVs.GetElementIDs())
					{
						const FVector2f UVvalue = UVCoordinates[UVID];
						int32 NewIndex = UVOverlay->AppendElement(FVector2f(UVvalue));
						UVIndexMap[UVID.GetValue()] = NewIndex;
					}

					// copy uv "index buffer"
					FUVID InvalidUVID(INDEX_NONE);
					TArray<int32> TrisMissingUVs; 
					TMap<FIndex2i, int> SplitUVMapping; // mapping from (ParentVID, OrigElID) -> SplitElID, for UV ElIDs that need to be split to multiple vertices
					for (int32 TriID : MeshOut.TriangleIndicesItr())
					{
						FTriangleID TriangleID = TriIDMap[TriID];
						// NB: the mesh description lacks a const method variant of this function, hence the const_cast. 
						// Don't change the UVIndices values!
						TArrayView<FUVID> UVIndices = const_cast<FMeshDescription*>(MeshIn)->GetTriangleUVIndices(TriangleID, UVLayerIndex);

						if (UVIndices[0] == InvalidUVID || UVIndices[1] == InvalidUVID || UVIndices[2] == InvalidUVID)
						{
							// keep track of the tris that don't have UVs in the mesh description shared UV channel.
							TrisMissingUVs.Add(TriID);
							continue;
						}

						// translate to Overlay indicies 
						FIndex3i TriUV( UVIndexMap[UVIndices[0].GetValue()],
							            UVIndexMap[UVIndices[1].GetValue()],
							            UVIndexMap[UVIndices[2].GetValue()] );
							
						///--  We have to do some clean-up on the shared UVs that come from MeshDecription --///
						// This clean up should go away if MeshDescription can solve these problems during import from the source fbx files
						{
							// Helper to split UVs out on a triangle while keeping UVs with the same source ID and parent vertex the same element in the overlay
							auto SplitVertexUV = [&SplitUVMapping, &UVCoordinates, &UVIndices, &UVOverlay](int ParentVID, FIndex3i& TriUV, int Idx)
							{
								FIndex2i SplitUVID(ParentVID, TriUV[Idx]);
								int* FoundUVEl = SplitUVMapping.Find(SplitUVID);
								if (FoundUVEl)
								{
									TriUV[Idx] = *FoundUVEl;
								}
								else
								{
									const FVector2f UVvalue = UVCoordinates[UVIndices[Idx]];
									int NewUVEl = UVOverlay->AppendElement(FVector2f(UVvalue));
									TriUV[Idx] = NewUVEl;
									SplitUVMapping.Add(SplitUVID, NewUVEl);
								}
							};

							// MeshDescription can attach multiple vertices to the same UV element.  DynamicMesh does not.
							// if we have already used this element for a different mesh vertex, split it.
							const FIndex3i ParentTriangle = MeshOut.GetTriangle(TriID);
							for (int i = 0; i < 3; ++i)
							{
								int32 ParentVID = UVOverlay->GetParentVertex(TriUV[i]);
								if (ParentVID != FDynamicMesh3::InvalidID && ParentVID != ParentTriangle[i])
								{
									SplitVertexUV(ParentTriangle[i], TriUV, i);
								}
							}

							// MeshDescription allows for degenerate UV tris.  Dynamic Mesh does not.
							// if the UV tri is degenerate we split the degenerate UV edge by adding a new UV
							// in its place, or if it is totally degenerate we add 2 new UVs

							if (TriUV[0] == TriUV[1] && TriUV[0] == TriUV[2])
							{
								const FVector2f UVvalue = UVCoordinates[UVIndices[1]];
								SplitVertexUV(ParentTriangle[1], TriUV, 1);
								SplitVertexUV(ParentTriangle[2], TriUV, 2);
							}
							else
							{
								int SplitWhich = -1;
								if (TriUV[0] == TriUV[1])
								{
									SplitWhich = 1;
								}
								else if (TriUV[0] == TriUV[2] || TriUV[1] == TriUV[2])
								{
									SplitWhich = 2;
								}
								if (SplitWhich > -1)
								{
									SplitVertexUV(ParentTriangle[SplitWhich], TriUV, SplitWhich);
								}
							}
						}

						// set the triangle in the overlay
						UVOverlay->SetTriangle(TriID, TriUV);
					}
				
					// If some of the mesh description triangles were missing shared UVs, 
					// use the per-vertex UVs on the mesh description and weld these.
					// NB: This can happen when multiple meshes with different UV layer count are "combined" during import
					// NB: these mesh description per-vertex UVs always exist, but may default to zero
					// we may want to revisit this in the future and not set UVs in the Overlay for these triangles,
					// but currently some of our tools (e.g. inspector) assume the every mesh triangle has an overlay triangle.
					if (TrisMissingUVs.Num() != 0)
					{
						FUVWelder UVWelder;
						UVWelder.UVOverlay = UVOverlay;
						for (int32 TriangleID : TrisMissingUVs)
						{
							const FIndex3i Tri = MeshOut.GetTriangle(TriangleID);
							const FTriData& TriData = AddedTriangles[TriangleID];
							FIndex3i TriUV;
							for (int j = 0; j < 3; ++j)
							{
								const FVector2f UV = InstanceUVs.Get(TriData.TriInstances[j], UVLayerIndex);
								TriUV[j] = UVWelder.FindOrAddUnique(UV, Tri[j]);
							}
							UVOverlay->SetTriangle(TriangleID, TriUV);
						}

					}
				}
			});
			Pending.Add(MoveTemp(UVTask));
		}


		if (NormalOverlay != nullptr)
		{
			UE::Tasks::FTask NormalTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&]()
			{
				FNormalWelder NormalWelder;
				NormalWelder.NormalOverlay = NormalOverlay;
				for (int32 TriangleID : MeshOut.TriangleIndicesItr())
				{
					const FIndex3i Tri = MeshOut.GetTriangle(TriangleID);
					const FTriData& TriData = AddedTriangles[TriangleID];
					FIndex3i TriNormals;
					for (int j = 0; j < 3; ++j)
					{
						const FVector3f Normal = InstanceNormals.Get(TriData.TriInstances[j]);
						TriNormals[j] = NormalWelder.FindOrAddUnique(Normal, Tri[j]);
					}
					NormalOverlay->SetTriangle(TriangleID, TriNormals);
				}
			});
			Pending.Add(MoveTemp(NormalTask));
		}


		if (TangentOverlay != nullptr)
		{
			auto TangentTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&]()
			{
				FNormalWelder TangentWelder;
				TangentWelder.NormalOverlay = TangentOverlay;
				for (int32 TriangleID : MeshOut.TriangleIndicesItr())
				{
					const FIndex3i Tri = MeshOut.GetTriangle(TriangleID);
					const FTriData& TriData = AddedTriangles[TriangleID];
					FIndex3i TriVector;
					for (int j = 0; j < 3; ++j)
					{
						const FVector3f Vector = InstanceTangents.Get(TriData.TriInstances[j]);
						TriVector[j] = TangentWelder.FindOrAddUnique(Vector, Tri[j]);
					}
					TangentOverlay->SetTriangle(TriangleID, TriVector);
				}
			});
			Pending.Add(MoveTemp(TangentTask));
		}

		if (BiTangentOverlay != nullptr)
		{
			auto BiTangentTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&]()
			{
				FNormalWelder BiTangentWelder;
				BiTangentWelder.NormalOverlay = BiTangentOverlay;
				for (int32 TriangleID : MeshOut.TriangleIndicesItr())
				{
					const FIndex3i Tri = MeshOut.GetTriangle(TriangleID);
					const FTriData& TriData = AddedTriangles[TriangleID];
					FIndex3i TriVector;
					for (int j = 0; j < 3; ++j)
					{
						// compute the bi tangent.
						const FVertexInstanceID VertexInstanceID = TriData.TriInstances[j];
						const FVector3f NormalVector = InstanceNormals.Get(VertexInstanceID);
						const FVector3f TangentVector = InstanceTangents.Get(VertexInstanceID);
						const float BiSign  = InstanceBiTangentSign.Get(VertexInstanceID);

						FVector3f BiTangentVector = BiSign * FVector3f::CrossProduct(NormalVector, TangentVector);
						BiTangentVector.Normalize();

						TriVector[j] = BiTangentWelder.FindOrAddUnique(BiTangentVector, Tri[j]);
					}
					BiTangentOverlay->SetTriangle(TriangleID, TriVector);
				}
			});
			Pending.Add(MoveTemp(BiTangentTask));
		}

		if (ColorOverlay != nullptr)
		{
			auto ColorTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&]()
			{
				const FVector4f DefaultColor4 = InstanceColors.GetDefaultValue();

				FColorWelder ColorWelder(ColorOverlay);
				for (int32 TriangleID : MeshOut.TriangleIndicesItr())
				{
					const FIndex3i Tri = MeshOut.GetTriangle(TriangleID);
					const FTriData& TriData = AddedTriangles[TriangleID];
					FIndex3i TriVector;
					for (int j = 0; j < 3; ++j)
					{
						FVector4f InstanceColor4 = InstanceColors.Get(TriData.TriInstances[j], 0);
						ApplyVertexColorTransform(InstanceColor4);
						
						const FVector4f OverlayColor(InstanceColor4.X, InstanceColor4.Y, InstanceColor4.Z, InstanceColor4.W);
						TriVector[j] = ColorWelder.FindOrAddUnique(OverlayColor, Tri[j]);

						// need to detect if the vertex instance color actually held any real data..
						bFoundNonDefaultVertexInstanceColor |= (InstanceColor4 != DefaultColor4);
					}
					ColorOverlay->SetTriangle(TriangleID, TriVector);
				}
			});
			Pending.Add(MoveTemp(ColorTask));
		}

		if (MaterialIDAttrib != nullptr)
		{
			auto MaterialTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&]()
			{
				for (int32 TriangleID : MeshOut.TriangleIndicesItr())
				{
					const FTriData& TriData = AddedTriangles[TriangleID];
					int32 MaterialIndex = TriData.PolygonGroupID;
					if (PolygonGroupToMaterialIndexMap.IsValidIndex(TriData.PolygonGroupID))
					{
						MaterialIndex = PolygonGroupToMaterialIndexMap[TriData.PolygonGroupID];
					}
					MaterialIDAttrib->SetValue(TriangleID, &MaterialIndex);
				}
			});
			Pending.Add(MoveTemp(MaterialTask));
		}

		for (const FSkinWeightsAttribCopyInfo& SkinWeightAttribInfo: SkinWeightAttribs)
		{
			auto SkinWeightsTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&, SkinWeightAttribInfo]() -> void
            {
                for (int32 VertexID: MeshOut.VertexIndicesItr())
                {
                	SkinWeightAttribInfo.DynaMeshAttribRef->SetValue(VertexID, SkinWeightAttribInfo.MeshDescAttribRef.Get(VertIDMap[VertexID]));
                }
				
            });
			Pending.Add(MoveTemp(SkinWeightsTask));
		}

		for (const FMorphTargetAttribCopyInfo& MorphTargetAttribInfo : MorphTargetAttribs)
		{
			auto MorphTargetTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&, MorphTargetAttribInfo]() -> void
			{
				for (int32 VertexID: MeshOut.VertexIndicesItr())
				{
					const FVector3f& V = MorphTargetAttribInfo.MeshDescAttribRef.Get(VertIDMap[VertexID]);
					MorphTargetAttribInfo.DynaMeshAttribRef->SetValue(VertexID, V);
				}
			});
			Pending.Add(MoveTemp(MorphTargetTask));
		}

		if (BoneNameAttrib != nullptr)
		{	
			auto BonesTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&]() -> void
			{
				const int NumBones = Attributes.GetNumBones();
				for (int32 BoneID = 0; BoneID < NumBones; ++BoneID)
				{
					BoneNameAttrib->SetValue(BoneID, Attributes.GetBoneNames().Get(BoneID));

					if (BoneParentIndexAttrib)
					{	
						BoneParentIndexAttrib->SetValue(BoneID, Attributes.GetBoneParentIndices().Get(BoneID));
					}

					if (BonePoseAttrib)
					{
						BonePoseAttrib->SetValue(BoneID, Attributes.GetBonePoses().Get(BoneID));
					}

					if (BoneColorAttrib)
					{
						BoneColorAttrib->SetValue(BoneID, Attributes.GetBoneColors().Get(BoneID));
					}
				}
				
			});
			Pending.Add(MoveTemp(BonesTask));
		}

		// initialize polygroup layers
		for (int32 GroupLayerIdx = 0; GroupLayerIdx < PolygroupAttribs.Num(); GroupLayerIdx++)
		{
			auto PolygroupTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&,Index=GroupLayerIdx]()
			{
				TTriangleAttributesConstRef<int32> InputGroupSet = PolygroupAttribs[Index];
				FDynamicMeshPolygroupAttribute* OutputGroupSet = MeshOut.Attributes()->GetPolygroupLayer(Index);
				if (ensure(OutputGroupSet))
				{
					OutputGroupSet->SetName(PolygroupAttribNames[Index]);
					for (int32 tid : MeshOut.TriangleIndicesItr())
					{
						FTriangleID SourceTriangleID = TriIDMap[tid];
						int32 GroupID = InputGroupSet.Get(SourceTriangleID);
						OutputGroupSet->SetValue(tid, &GroupID);
					}
				}
			});
			Pending.Add(MoveTemp(PolygroupTask));
		}

		// initialize triangle labels
		for (const FTriangleLabelAttribCopyInfo& TriLabelAttrInfo: TriangleLabelAttributes)
		{
			auto TriangleLabelTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&, TriLabelAttrInfo]()
			{
				if (ensure(TriLabelAttrInfo.DynaMeshAttribRef))
				{
					for (int32 tid : MeshOut.TriangleIndicesItr())
					{
						FTriangleID SourceTriangleID = TriIDMap[tid];
						TriLabelAttrInfo.DynaMeshAttribRef->SetValue(tid, TriLabelAttrInfo.MeshDescAttribRef.Get(SourceTriangleID));
					}
				}
			});
			Pending.Add(MoveTemp(TriangleLabelTask));
		}

		// initialize weight layers
		for (int32 WeightLayerIdx = 0; WeightLayerIdx < WeightAttribs.Num(); WeightLayerIdx++)
		{
			auto WeightTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, &WeightAttribs, &WeightAttribNames, &MeshOut, WeightLayerIdx]()
			{
				TVertexAttributesConstRef<float> InputWeights = WeightAttribs[WeightLayerIdx];
				FDynamicMeshWeightAttribute* OutputWeights = MeshOut.Attributes()->GetWeightLayer(WeightLayerIdx);
				if (ensure(OutputWeights))
				{
					OutputWeights->SetName(WeightAttribNames[WeightLayerIdx]);
					for (int32 vid : MeshOut.VertexIndicesItr())
					{
						FVertexID SourceVertexID = VertIDMap[vid];
						float Value = InputWeights.Get(SourceVertexID);
						OutputWeights->SetValue(vid, &Value);
					}
				}
			});
			Pending.Add(MoveTemp(WeightTask));
		}

		// wait for all work to be done
		UE::Tasks::Wait(Pending);

		// add non-manifold mapping information if required.
		if (!bSrcIsManifold && bVIDsFromNonManifoldMeshDescriptionAttr)
		{
			TArray<int32> TmpVIDMap;
			TmpVIDMap.SetNumUninitialized(VertIDMap.Num());
			for (int32 i = 0, I = VertIDMap.Num(); i < I; ++i)
			{
				TmpVIDMap[i] = VertIDMap[i].GetValue();
			}
			FNonManifoldMappingSupport::AttachNonManifoldVertexMappingData(TmpVIDMap, MeshOut);
		}


		if (!bFoundNonDefaultVertexInstanceColor)
		{
			MeshOut.Attributes()->DisablePrimaryColors();
		}
	}

	// free maps if no longer needed
	if (!bCalculateMaps)
	{
		TriIDMap.Empty();
		VertIDMap.Empty();
	}

	FDateTime Time_AfterAttribs = FDateTime::Now();

	if (bPrintDebugMessages)
	{
		UE_LOGF(LogTemp, Warning, "FMeshDescriptionToDynamicMesh:  Conversion Timing: Triangles %fs   Attributbes %fs",
			(Time_AfterTriangles - Time_AfterVertices).GetTotalSeconds(), (Time_AfterAttribs - Time_AfterTriangles).GetTotalSeconds());

		int NumUVs = (MeshOut.HasAttributes() && NumUVLayers > 0) ? MeshOut.Attributes()->PrimaryUV()->MaxElementID() : 0;
		int NumNormals = (NormalOverlay != nullptr) ? NormalOverlay->MaxElementID() : 0;
		UE_LOGF(LogTemp, Warning, "FMeshDescriptionToDynamicMesh:  FDynamicMesh verts %d triangles %d (primary) uvs %d normals %d", MeshOut.MaxVertexID(), MeshOut.MaxTriangleID(), NumUVs, NumNormals);
	}

}



template<typename RealType>
static void CopyTangents_Internal(const FMeshDescription* SourceMesh, const FDynamicMesh3* TargetMesh, TMeshTangents<RealType>* TangentsOut, const TArray<FTriangleID>& TriIDMap)
{

	FStaticMeshConstAttributes Attributes(*SourceMesh);

	TArrayView<const FVector3f> InstanceNormals = Attributes.GetVertexInstanceNormals().GetRawArray();
	TArrayView<const FVector3f> InstanceTangents = Attributes.GetVertexInstanceTangents().GetRawArray();
	TArrayView<const float> InstanceSigns = Attributes.GetVertexInstanceBinormalSigns().GetRawArray();

	if (!ensureMsgf(InstanceNormals.Num() != 0, TEXT("Cannot CopyTangents from MeshDescription with invalid Instance Normals"))) return;
	if (!ensureMsgf(InstanceTangents.Num() != 0, TEXT("Cannot CopyTangents from MeshDescription with invalid Instance Tangents"))) return;
	if (!ensureMsgf(InstanceSigns.Num() != 0, TEXT("Cannot CopyTangents from MeshDescription with invalid Instance BinormalSigns"))) return;

	TangentsOut->SetMesh(TargetMesh);
	TangentsOut->InitializeTriVertexTangents(false);
	
	for (int32 TriID : TargetMesh->TriangleIndicesItr())
	{

		const FTriangleID TriangleID = TriIDMap[TriID];
		TArrayView<const FVertexInstanceID> InstanceTri = SourceMesh->GetTriangleVertexInstances(TriangleID);
		for (int32 j = 0; j < 3; ++j)
		{
			FVector Normal(InstanceNormals[InstanceTri[j]]);
			FVector Tangent(InstanceTangents[InstanceTri[j]]);
			float BitangentSign = InstanceSigns[InstanceTri[j]];
			TVector<RealType> Bitangent = VectorUtil::Bitangent((TVector<RealType>)Normal, (TVector<RealType>)Tangent, (RealType)BitangentSign);
			Tangent.Normalize();
			UE::Geometry::Normalize(Bitangent);
			TangentsOut->SetPerTriangleTangent(TriID, j, (TVector<RealType>)Tangent, (TVector<RealType>)Bitangent);
		}
	}
}



void FMeshDescriptionToDynamicMesh::CopyTangents(const FMeshDescription* SourceMesh, const FDynamicMesh3* TargetMesh, TMeshTangents<float>* TangentsOut)
{
	if (!ensureMsgf(bCalculateMaps, TEXT("Cannot CopyTangents unless Maps were calculated"))) return;
	int32 TargetTriCount = TargetMesh->TriangleCount();
	if (!ensureMsgf(TriIDMap.Num() == TargetTriCount, TEXT("Tried to CopyTangents to mesh with different triangle count"))) return;
	CopyTangents_Internal<float>(SourceMesh, TargetMesh, TangentsOut, TriIDMap);
}



void FMeshDescriptionToDynamicMesh::CopyTangents(const FMeshDescription* SourceMesh, const FDynamicMesh3* TargetMesh, TMeshTangents<double>* TangentsOut)
{
	if (!ensureMsgf(bCalculateMaps, TEXT("Cannot CopyTangents unless Maps were calculated"))) return;
	if (!ensureMsgf(TriIDMap.Num() == TargetMesh->TriangleCount(), TEXT("Tried to CopyTangents to mesh with different triangle count"))) return;
	CopyTangents_Internal<double>(SourceMesh, TargetMesh, TangentsOut, TriIDMap);
}

void FMeshDescriptionToDynamicMesh::ApplyVertexColorTransform(FVector4f& Color) const
{
	if (bTransformVertexColorsLinearToSRGB)
	{
		// The corollary to bTransformVertexColorsSRGBToLinear in DynamicMeshToMeshDescription.
		// See DynamicMeshToMeshDescription::ApplyVertexColorTransform(..).
		//
		// StaticMeshes store vertex colors as FColor. The StaticMesh build always encodes
		// FColors as SRGB to ensure a good distribution of float values across the 8-bit range.
		//
		// Since there is no currently defined gamma space convention for vertex colors in
		// engine, an option is provided to pre-transform vertex colors (SRGB To Linear) when
		// writing out a MeshDescription.
		//
		// We similarly provide the inverse of that optional pre-transformation to maintain
		// color space consistency in our usage.
		LinearColors::LinearToSRGB(Color);
	}
}
