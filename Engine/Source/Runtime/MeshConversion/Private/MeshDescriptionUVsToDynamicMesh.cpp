// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshDescriptionUVsToDynamicMesh.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "MeshDescription.h"
#include "StaticMeshAttributes.h"

using namespace UE::Geometry;

// TODO: Need to test both conversions some more with meshes that only have instanced UVs.
// Also, we should probably keep maps for elements we split due to manifoldness issues, etc,
// rather than using all those heuristics to try to weld things back together on the conversion
// back.

namespace MeshDescriptionUVsToDynamicMeshLocals
{
	bool ShouldConversionUseSharedUVs(const FMeshDescription* MeshDescription, int32 LayerIndex)
	{
		const int32 NumSharedUVLayers = MeshDescription->GetNumUVElementChannels();
		return NumSharedUVLayers > LayerIndex && MeshDescription->UVs(LayerIndex).GetArraySize() != 0;
	}

	// These determine the mapping between UV values and the resulting mesh vertex positions. If we're
	// looking down on the unwrapped mesh, with the Z axis towards us, we want U's to be right, and
	// V's to be up. In Unreal's left-handed coordinate system, this means that we map U's to world Y
	// and V's to world X.
	// Also, Unreal changes the V coordinates of imported meshes to 1-V internally, and we undo this
	// while displaying the UV's because the users likely expect to see the original UV's (it would
	// be particularly confusing for users working with UDIM assets, where internally stored V's 
	// frequently end up negative).
	// The ScaleFactor just scales the mesh up. Scaling the mesh up makes it easier to zoom in
	// further into the display before getting issues with the camera near plane distance.
	FORCEINLINE FVector3d UVToVertPosition(const FVector2f& UV, double ScaleFactor)
	{
		return FVector3d((1 - UV.Y) * ScaleFactor, UV.X * ScaleFactor, 0);
	}
	FORCEINLINE FVector2f VertPositionToUV(const FVector3d& VertPosition, double ScaleFactor)
	{
		return FVector2f(VertPosition.Y / ScaleFactor, 1 - (VertPosition.X / ScaleFactor));
	}

	// This class is copied from MeshDescriptionToDynamicMesh.cpp. We could consider putting it
	// somewhere common.
	struct FVertexUV
	{
		int vid;
		float x;
		float y;
		bool operator==(const FVertexUV& o) const
		{
			return vid == o.vid && x == o.x && y == o.y;
		}
	};
	FORCEINLINE uint32 GetTypeHash(const FVertexUV& Vector)
	{
		// ugh copied from FVector clearly should not be using CRC for hash!!
		return FCrc::MemCrc32(&Vector, sizeof(Vector));
	}

	// This class is modeled on FUVWelder in MeshDescriptionToDynamicMesh.cpp,
	// except it adds to the dynamic mesh vertices instead of the mesh UV's.
	// Note: we could consider welding UV's differently by sorting all the verts and welding in one pass.
	class FUVMeshWelder
	{
	public:

		FUVMeshWelder(FDynamicMesh3* MeshToAddTo, double ScaleFactorIn)
			: Mesh(MeshToAddTo)
			, ScaleFactor(ScaleFactorIn)
		{
			check(MeshToAddTo && ScaleFactorIn != 0);
		}

		int FindOrAddUnique(const FVector2f& UV, int VertexID)
		{
			FVertexUV VertUV = { VertexID, UV.X, UV.Y };

			const int32* FoundIndex = UniqueVertexUVs.Find(VertUV);
			if (FoundIndex != nullptr)
			{
				return *FoundIndex;
			}

			int32 NewIndex = Mesh->AppendVertex(UVToVertPosition(UV, ScaleFactor));
			UniqueVertexUVs.Add(VertUV, NewIndex);
			return NewIndex;
		}

	protected:
		FDynamicMesh3* Mesh;
		double ScaleFactor;

		TMap<FVertexUV, int> UniqueVertexUVs;
	};
}


int32 FMeshDescriptionUVsToDynamicMesh::GetNumUVLayers(const FMeshDescription* MeshDescription) const
{
	FStaticMeshConstAttributes Attributes(*MeshDescription);
	return Attributes.GetVertexInstanceUVs().GetNumChannels();
}

TSharedPtr<FDynamicMesh3> FMeshDescriptionUVsToDynamicMesh::GetUVMesh(const FMeshDescription* MeshDescription)
{

	using namespace MeshDescriptionUVsToDynamicMeshLocals;

	check(ScaleFactor != 0);

	bool bUseSharedUVs = ShouldConversionUseSharedUVs(MeshDescription, UVLayerIndex);

	// The output that we'll return
	TSharedPtr<FDynamicMesh3> MeshOut = MakeShared<FDynamicMesh3>();

	if (bUseSharedUVs)
	{
		// Create vertices for the UV elements.
		const FUVArray& SharedUVs = MeshDescription->UVs(UVLayerIndex);
		TUVAttributesRef<const FVector2f> SharedUVCoordinates = SharedUVs.GetAttributes().GetAttributesRef<FVector2f>(MeshAttribute::UV::UVCoordinate);

		MeshOut->BeginUnsafeVerticesInsert();
		for (FUVID UVID : SharedUVs.GetElementIDs())
		{
			const FVector2f UV = SharedUVCoordinates[UVID];
			MeshOut->InsertVertex(UVID.GetValue(), UVToVertPosition(UV, ScaleFactor), true);
		}
		MeshOut->EndUnsafeVerticesInsert();
	}

	// Instance UV's and a welder are used if bUseSharedUVs is false, or if it is true but some 
	// verts are not initialized.
	FStaticMeshConstAttributes Attributes(*MeshDescription);
	TVertexInstanceAttributesConstRef<FVector2f> InstanceUVs = Attributes.GetVertexInstanceUVs();
	FUVMeshWelder UVMeshWelder(MeshOut.Get(), ScaleFactor);

	// Go through triangles and build our mesh
	MeshOut->BeginUnsafeTrianglesInsert();
	for (const FTriangleID TriangleID : MeshDescription->Triangles().GetElementIDs())
	{
		// The Tid must stay the same in the dynamic mesh for us to be able to bake things back
		int32 Tid = TriangleID.GetValue();

		// Will be populated with Vids in the dynamic mesh
		FIndex3i TriangleToInsert;

		// Used on instance UVs to map them to shared UV elements (vertices) in the dynamic mesh. 
		auto MakeWeldedUVsForTri = [this, MeshDescription, &UVMeshWelder, &InstanceUVs](const FTriangleID& TriangleIDIn, FIndex3i& TriangleToInsertOut) {
			
			TArrayView<const FVertexID> SourceTriangleVids = MeshDescription->GetTriangleVertices(TriangleIDIn);
			TArrayView<const FVertexInstanceID> SourceTriangleInstanceIds = MeshDescription->GetTriangleVertexInstances(TriangleIDIn);
			for (int32 i = 0; i < 3; ++i)
			{
				FVector2f UV = InstanceUVs.Get(SourceTriangleInstanceIds[i], UVLayerIndex);
				TriangleToInsertOut[i] = UVMeshWelder.FindOrAddUnique(UV, SourceTriangleVids[i].GetValue());
			}
		};

		if (!bUseSharedUVs)
		{
			// If we're not using shared UV's, each vertex instance has its own UV element. We weld them per vertex
			// if the UV's match.
			MakeWeldedUVsForTri(TriangleID, TriangleToInsert);
		}
		else
		{
			TArrayView<FUVID> TriUVIndices = MeshDescription->GetTriangleUVIndices(TriangleID, UVLayerIndex);

			// If some of the mesh description triangles were missing shared UVs,
			// use the per-vertex UVs on the mesh description and weld these.
			// NB: This can happen when multiple meshes with different UV layer count are "combined" during import
			// NB: these mesh description per-vertex UVs always exist, but may default to zero
			FUVID InvalidUVID(INDEX_NONE);
			if (TriUVIndices[0] == InvalidUVID || TriUVIndices[1] == InvalidUVID || TriUVIndices[2] == InvalidUVID)
			{
				MakeWeldedUVsForTri(TriangleID, TriangleToInsert);
			}
			else
			{
				// Triangle seems ok so far. We already added vertices for it previously, so set up the tri to add.
				// We'll be doing more checks for degeneracy and such before inserting.
				for (int32 i = 0; i < 3; ++i)
				{
					TriangleToInsert[i] = TriUVIndices[i].GetValue();
				}
			}
		}

		// Our splitter, returns the new vertex id
		auto SplitDynamicMeshVertex = [MeshOut](int32 Vid) {
			check(MeshOut->IsVertex(Vid));
			const FVector3d Position = MeshOut->GetVertex(Vid);
			return MeshOut->AppendVertex(Position);
		};

		// Split vertices if our triangle turned out to be degenerate.
		if (TriangleToInsert[0] == TriangleToInsert[1] || TriangleToInsert[0] == TriangleToInsert[2])
		{
			TriangleToInsert[0] = SplitDynamicMeshVertex(TriangleToInsert[0]);
		}
		if (TriangleToInsert[1] == TriangleToInsert[2])
		{
			TriangleToInsert[1] = SplitDynamicMeshVertex(TriangleToInsert[1]);
		}

		// Attempt triangle insertion
		EMeshResult Result = MeshOut->InsertTriangle(Tid, TriangleToInsert, 0, true);

		// Duplicate vertices if insertion failed due to a non-manifold edge
		if (Result == EMeshResult::Failed_WouldCreateNonmanifoldEdge)
		{
			bool bDuplicate[3] = { false, false, false };

			int e0 = MeshOut->FindEdge(TriangleToInsert[0], TriangleToInsert[1]);
			int e1 = MeshOut->FindEdge(TriangleToInsert[1], TriangleToInsert[2]);
			int e2 = MeshOut->FindEdge(TriangleToInsert[2], TriangleToInsert[0]);

			// determine which verts need to be duplicated by seeing whether existing edges already
			// have two triangles.
			if (e0 != FDynamicMesh3::InvalidID && MeshOut->IsBoundaryEdge(e0) == false)
			{
				bDuplicate[0] = true;
				bDuplicate[1] = true;
			}
			if (e1 != FDynamicMesh3::InvalidID && MeshOut->IsBoundaryEdge(e1) == false)
			{
				bDuplicate[1] = true;
				bDuplicate[2] = true;
			}
			if (e2 != FDynamicMesh3::InvalidID && MeshOut->IsBoundaryEdge(e2) == false)
			{
				bDuplicate[2] = true;
				bDuplicate[0] = true;
			}

			// Do the actual duplication
			for (int32 i = 0; i < 3; ++i)
			{
				if (bDuplicate[i])
				{
					TriangleToInsert[i] = SplitDynamicMeshVertex(TriangleToInsert[i]);
				}
			}

			// Reinsert the triangle
			Result = MeshOut->InsertTriangle(Tid, TriangleToInsert, 0, true);
		}

		check(Result == EMeshResult::Ok || Result == EMeshResult::Failed_TriangleAlreadyExists);
	}//end iterating through triangles

	MeshOut->EndUnsafeTrianglesInsert();

	return MeshOut;
}

void FMeshDescriptionUVsToDynamicMesh::BakeBackUVsFromUVMesh(const FDynamicMesh3* DynamicMesh, FMeshDescription* MeshDescription) const
{
	using namespace MeshDescriptionUVsToDynamicMeshLocals;

	check(ScaleFactor != 0);
	check(DynamicMesh->TriangleCount() == MeshDescription->Triangles().Num())

	// Regardless of whether we have shared UVs or not, we are supposed to update the instance UVs.
	// This is straightforward, since any welding or splitting does not change anything.
	FStaticMeshAttributes Attributes(*MeshDescription);
	TVertexInstanceAttributesRef<FVector2f> InstanceUVs = Attributes.GetVertexInstanceUVs();
	for (int32 Tid : DynamicMesh->TriangleIndicesItr())
	{
		TArrayView<const FVertexInstanceID> MeshDescriptionTriInstanceIds = MeshDescription->GetTriangleVertexInstances(Tid);
		FIndex3i DynamicMeshTriVids = DynamicMesh->GetTriangle(Tid);
		for (int i = 0; i < 3; ++i)
		{
			FVector3d VertPosition = DynamicMesh->GetVertex(DynamicMeshTriVids[i]);
			InstanceUVs.Set(MeshDescriptionTriInstanceIds[i], UVLayerIndex, VertPositionToUV(VertPosition, ScaleFactor));
		}
	}

	if (ShouldConversionUseSharedUVs(MeshDescription, UVLayerIndex))
	{
		// For shared UVs we may have to deal with splitting and welding, which makes things a bit more
		// complicated.

		// VidToUVElement is able to map vids back to newly created or merged UV elements. It
		// also lets us know if we've dealt with a specific vid before.
		TArray<int32> VidToUVElement;
		VidToUVElement.SetNum(DynamicMesh->MaxVertexID());
		for (int32& Value : VidToUVElement)
		{
			Value = INDEX_NONE;
		}

		FUVArray& SharedUvs = MeshDescription->UVs(UVLayerIndex);
		TUVAttributesRef<FVector2f> SharedUVCoordinates = SharedUvs.GetAttributes().GetAttributesRef<FVector2f>(MeshAttribute::UV::UVCoordinate);

		MeshDescription->SuspendUVIndexing();

		for (int32 Tid : DynamicMesh->TriangleIndicesItr())
		{
			// Look at the UV ID's this triangle had, and the vertex ID's it now has. Vertex ID's correspond
			// to UV element ID's except in cases where a new UV element needs to be created.
			TArrayView<FUVID> TriUVIDs = MeshDescription->GetTriangleUVIndices(FTriangleID(Tid), UVLayerIndex);
			FIndex3i TriVids = DynamicMesh->GetTriangle(Tid);

			bool bTriUVIDsChanged = false;

			// Examine the triangle vertices
			for (int i = 0; i < 3; ++i)
			{
				// By examining the current and expected Vids, we can tell whether the UV connectivity changed
				int32 Vid = TriVids[i];
				int32 ExpectedVid = TriUVIDs[i].GetValue();

				// First check whether we've dealt with this vert before
				if (VidToUVElement[Vid] != INDEX_NONE)
				{
					// We still need to update triangle connectivity unless it corresponds to the existing expected vert
					// (this includes verts that we split for conversion only, since those will map to the expected vert)
					if (VidToUVElement[Vid] != ExpectedVid)
					{
						TriUVIDs[i] = VidToUVElement[Vid];
						bTriUVIDsChanged = true;
					}
					continue;
				}

				// If we haven't dealt with this vert, we will need to update its entry in SharedUVCoordinates.

				FVector3d VertPosition = DynamicMesh->GetVertex(Vid);
				FVector2f NewUVValue = VertPositionToUV(VertPosition, ScaleFactor);
				
				// See if the vert is the same one that we were expecting (including one that we may have split off just for
				// the conversion, in which case the location will match).
				if (Vid == ExpectedVid ||
					(DynamicMesh->IsVertex(ExpectedVid) && DynamicMesh->GetVertex(ExpectedVid) == VertPosition))
				{
					VidToUVElement[Vid] = ExpectedVid;
					SharedUVCoordinates.Set(TriUVIDs[i], NewUVValue);
					// No need to update TriUVIDs
					continue;
				}

				// See if the vert corresponds to an element that exists in SharedUVCoordinates
				if (MeshDescription->IsUVValid(FUVID(Vid), UVLayerIndex))
				{
					VidToUVElement[Vid] = Vid;
					SharedUVCoordinates.Set(FUVID(Vid), NewUVValue);
					TriUVIDs[i] = FUVID(Vid);
					bTriUVIDsChanged = true;
					continue;
				}

				// Otherwise, this is a new element we need to create, and we need to update TriUVIDs
				TriUVIDs[i] = MeshDescription->CreateUV(UVLayerIndex);
				VidToUVElement[Vid] = TriUVIDs[i].GetValue();
				SharedUVCoordinates.Set(TriUVIDs[i], NewUVValue);
				bTriUVIDsChanged = true;
			}//end dealing with tri verts

			if (bTriUVIDsChanged)
			{
				MeshDescription->SetTriangleUVIndices(Tid, TriUVIDs, UVLayerIndex);
			}
		}//end iterating through triangles

		MeshDescription->ResumeUVIndexing();
	}
}