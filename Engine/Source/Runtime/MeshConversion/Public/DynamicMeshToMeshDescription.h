// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "GeometryBase.h"
#include "MeshConversionOptions.h"
#include "MeshDescription.h"

PREDECLARE_GEOMETRY(template<typename RealType> class TMeshTangents);
namespace UE { namespace Geometry { class FDynamicMesh3; } }
struct FMeshDescription;
struct FTriangleID;
struct FVertexID;

using UE::Geometry::FDynamicMesh3;

/**
 * Convert FDynamicMesh3 to FMeshDescription
 *
 */
class FDynamicMeshToMeshDescription
{
public:
	/** If true, will print some possibly-helpful debugging spew to output log */
	bool bPrintDebugMessages = false;

	/** General settings for conversions to mesh description */
	FConversionToMeshDescriptionOptions ConversionOptions;

	// Set MaterialID to Polygroup mapping via the reverse mapping
	void MESHCONVERSION_API SetMaterialIDMapFromInverseMap(TArrayView<const int32> PolygroupIDToMaterialIDMap);

	FDynamicMeshToMeshDescription()
	{
	}

	FDynamicMeshToMeshDescription(FConversionToMeshDescriptionOptions ConversionOptions) : ConversionOptions(ConversionOptions)
	{
	}

	/**
	 * Checks if element counts match. If false then Update can't be called -- you must call Convert
	 *
	 * @param DynamicMesh The dynamic mesh with updated vertices or attributes
	 * @param MeshDescription The corresponding mesh description
	 * @param bVerticesOnly If true, only check vertex counts match
	 * @param bAttributesOnly If true, only check what needs to be checked for UpdateAttributes
							 (will check vertices or triangles depending on whether attributes are per vertex or in overlays)
	 */
	static MESHCONVERSION_API bool HaveMatchingElementCounts(const FDynamicMesh3* DynamicMesh, const FMeshDescription* MeshDescription, bool bVerticesOnly, bool bAttributesOnly);

	/**
	 * Checks if element counts match. If false then Update can't be called -- you must call Convert
	 * Result is based on the current ConversionOptions (e.g. if you are only updating vertices, mismatched triangles are ok)
	 *
	 * @param DynamicMesh The dynamic mesh with updated vertices or attributes
	 * @param MeshDescription The corresponding mesh description
	 */
	MESHCONVERSION_API bool HaveMatchingElementCounts(const FDynamicMesh3* DynamicMesh, const FMeshDescription* MeshDescription);

	/**
	 * Default conversion of DynamicMesh to MeshDescription. Calls functions below depending on mesh state
	 *
	 * @param bCopyTangents - should tangent, bitangent overlays exist on the MeshIn, this will use them when producing the MeshDescription MeshOut
	 *                         
	 * Note: Don't copy tangents if the resulting MeshDescription corresponds to a StaticMesh with autogenerated tangents
	 *       in this case the MeshDescription is expected to have empty tangents.
	 *       
	 */
	MESHCONVERSION_API void Convert(const FDynamicMesh3* MeshIn, FMeshDescription& MeshOut, bool bCopyTangents = false);

	/**
	 * Updates the given mesh description based conversion options provided in the constructor. Assumes
	 * the mesh topology has not changed. 
	 * Annoyingly, this can't just be named Update() due to ambiguity with the function below, which
	 * existed beforehand and should probably have been this function instead.
	 */
	MESHCONVERSION_API void UpdateUsingConversionOptions(const FDynamicMesh3* MeshIn, FMeshDescription& MeshOut);

	/**
	 * Update existing MeshDescription based on DynamicMesh. Assumes mesh topology has not changed.
	 * Copies positions 
	 * optionally, normals, tangents and UVs
	 */
	MESHCONVERSION_API void Update(const FDynamicMesh3* MeshIn, FMeshDescription& MeshOut, bool bUpdateNormals = true, bool bUpdateTangents = false, bool bUpdateUVs = false);
	

	/**
	 * Update only attributes, assuming the mesh topology has not changed.  Does not touch positions.
	 *	NOTE: assumes the order of triangles in the MeshIn correspond to the ordering you'd get by iterating over triangles, on MeshOut
	 *		  This matches conversion currently used in MeshDescriptionToDynamicMesh.cpp, but if that changes we will need to change this function to match!
	 *  @param bUpdateNormals   Specifies if the normals should be transfered from the MeshIn PrimaryNormal overlay if it exists
	 *  @param bUpdateTangents  Specifies if the tangent and bitangent sign should be populated from the MeshIn PrimaryTangents and PrimaryBiTangents. 
	 *                          this requires the PrimaryNormals to exist as well. 
	 *  @param bUpdateUVs       Specifices if the UV layers should be transfered from the MeshIn overlays.
	 */
	MESHCONVERSION_API void UpdateAttributes(const FDynamicMesh3* MeshIn, FMeshDescription& MeshOut, bool bUpdateNormals, bool bUpdateTangents, bool bUpdateUVs);

	/**
	 * Use the TMeshTangents to update the Tangent and BinormalSign attributes of the MeshDescription, assuming mesh topology has not changed. Does not modify any other attributes.
	 *  NOTE: this ignores any tangent or bitangent overlays on the MeshIn, and instead uses the tangent and bitangent information
	 *        stored in the TMeshTangents
	 *	NOTE: assumes the order of triangles in the MeshIn correspond to the ordering you'd get by iterating over triangles, on MeshOut
	 *		  This matches conversion currently used in MeshDescriptionToDynamicMesh.cpp, but if that changes we will need to change this function to match!
	 */
	MESHCONVERSION_API void UpdateTangents(const FDynamicMesh3* MeshIn, FMeshDescription& MeshOut, const UE::Geometry::TMeshTangents<double>* SrcTangents);


	/**
	 * Use the MeshIn Overlays to update the Tangent and BinormalSign attributes of the MeshDescription, assuming mesh topology has not changed. Does not modify any other attributes.
	 *	NOTE: assumes the order of triangles in the MeshIn correspond to the ordering you'd get by iterating over triangles, on MeshOut
	 *		  This matches conversion currently used in MeshDescriptionToDynamicMesh.cpp, but if that changes we will need to change this function to match!
	 */
	MESHCONVERSION_API void UpdateTangents(const FDynamicMesh3* MeshIn, FMeshDescription& MeshOut);


	/**
	 * Update only vertex colors, assuming the mesh topology has not changed.  Does not touch positions or other attributes.
	 *	NOTE: assumes the order of triangles in the MeshIn correspond to the ordering you'd get by iterating over triangles, on MeshOut
	 *		  This matches conversion currently used in MeshDescriptionToDynamicMesh.cpp, but if that changes we will need to change this function to match!
	 */
	MESHCONVERSION_API void UpdateVertexColors(const FDynamicMesh3* MeshIn, FMeshDescription& MeshOut);



	//
	// Internal functions that you can also call directly
	//

	/**
	 * Ignore any Attributes on input Mesh, calculate per-vertex normals and have MeshDescription compute tangents.
	 * One VertexInstance per input vertex is generated
	 */
	MESHCONVERSION_API void Convert_NoAttributes(const FDynamicMesh3* MeshIn, FMeshDescription& MeshOut);

	/**
	 * Convert while minimizing VertexInstance count, IE new VertexInstances are only created 
	 * if a unique UV or Normal is required.
	 * 
	 * Note: This doesn't copy any tangents from the FDynamicMesh
	 * Note: This conversion is not currently being used. It is unclear if all consumers of FMeshDescription can handle such shared vertex instances 
	 */
	MESHCONVERSION_API void Convert_SharedInstances(const FDynamicMesh3* MeshIn, FMeshDescription& MeshOut);

	/**
	 * Convert with no shared VertexInstances. A new VertexInstance is created for
	 * each triangle vertex (ie corner). However vertex positions are shared and
     * the shared UVs structures are populated.
	 * 
	 * @param bCopyTangents - should tangent, bitangent overlays exist on the MeshIn, this will use them when producing the MeshDescription MeshOut
	 *
	 * Note: Don't copy tangents if the resulting MeshDescription corresponds to a StaticMesh with autogenerated tangents
	 *       in this case the MeshDescription is expected to have empty tangents.
	 */
	MESHCONVERSION_API void Convert_NoSharedInstances(const FDynamicMesh3* MeshIn, FMeshDescription& MeshOut, bool bCopyTangents);



protected:

	/**
	 * Transfer PolygroupLayers from DynamicMesh AttributeSet to MeshDescription.
	 * Will copy to existing MeshDescription TriangleAttribute<int32> if one with the same name exists.
	 * Otherwise will register a new one.
	 */
	MESHCONVERSION_API void ConvertPolygroupLayers(const FDynamicMesh3* MeshIn, FMeshDescription& MeshOut, const TArray<FTriangleID>& IndexToTriangleIDMap);

	/**
	* Transfer WeightLayers from DynamicMesh AttributeSet to MeshDescription.
	* Will copy to existing MeshDescription VertexAttribute<float> if one with the same name exists.
	* Otherwise will register a new one.
	*/
	MESHCONVERSION_API void ConvertWeightLayers(const FDynamicMesh3* MeshIn, FMeshDescription& MeshOut, const TArray<FVertexID>& IndexToVertexIDMap);

	/**
	 * Transfer triangle label attributes from DynamicMesh AttributeSet to MeshDescription.
	 * Will copy to existing MeshDescription TriangleAttribute<FName> if one with the same name exists.
	 * Otherwise will register a new one.
	 */
	UE_INTERNAL void ConvertTriangleAttributes(const FDynamicMesh3* MeshIn, FMeshDescription& MeshOut, const TArray<FTriangleID>& IndexToTriangleIDMap);
	
	/**
	 * Applies an optional sRGB-to-Linear color transform on the input. The color transform
	 * is controlled by ConversionOptions.bTransformVtxColorsSRGBToLinear.
	 *
	 * The counterpart to this method is MeshDescriptionToDynamicMesh::ApplyVertexColorTransform
	 * which will undo this color transformation when the MeshDescription is read back.
	 *
	 * @param Color color to transform
	 */
	MESHCONVERSION_API void ApplyVertexColorTransform(FVector4f& Color) const;
	
private:
	/** Optional remapping from dynamic mesh material ID to desired mesh description polygroup IDs. IDs not found in mapping will be mapped to themselves. */
	TArray<int32> MaterialIDToPolygroupIDMap;
};
