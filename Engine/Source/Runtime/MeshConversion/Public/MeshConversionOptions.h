// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


struct FConversionToMeshDescriptionOptions
{
public:
	/** Should triangle groups be transfered to MeshDescription via custom PolyTriGroups attribute */
	bool bSetPolyGroups = true;


	/** Should Positions of vertices in MeshDescription be updated */
	bool bUpdatePositions = true;

	/** Should normals of MeshDescription be updated, if available and relevant */
	bool bUpdateNormals = true;

	/** Should Tangents and BiTangentSign of MeshDescription be updated, if available and relevant */
	bool bUpdateTangents = false;

	/** Should UVs of MeshDescription be updated, if available and relevant */
	bool bUpdateUVs = false;

	/** Should Vertex Colors of MeshDescription be updated, if available and relevant */
	bool bUpdateVtxColors = false;

	/** Should Vertex Colors of MeshDescription be transformed from SRGB to Linear */
	bool bTransformVtxColorsSRGBToLinear = true;

	/** Should the mesh be put back in the original non-manifold state, if the information is present */
	bool bConvertBackToNonManifold = false;

	/** Should normal seams be tagged as hard edges in the mesh description */
	bool bSetHardEdgesFromNormalSeams = true;

	/** If non-negative, sets the max valid material ID to assign -- IDs higher than this will be reset to 0. Ignored if negative. */
	int32 MaxValidMaterialID = INDEX_NONE;

	//
	// utility functions for common configuration cases
	//

	void SetToVertexColorsOnly()
	{
		bSetPolyGroups = bUpdatePositions = bUpdateNormals = bUpdateTangents = bUpdateUVs = false;
		bUpdateVtxColors = true;
	}
};
