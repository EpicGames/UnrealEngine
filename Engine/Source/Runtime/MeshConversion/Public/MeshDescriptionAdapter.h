// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshDescription.h"
#include "StaticMeshAttributes.h"
#include "Spatial/MeshAABBTree3.h"
#include "MeshAdapter.h"


/**
 * Basic struct to adapt a FMeshDescription for use by GeometryProcessing classes that template the mesh type and expect a standard set of basic accessors
 * For example, this adapter will let you use a FMeshDescription with GeometryProcessing's TMeshAABBTree3
 * See also the Editable version below
 *
 *  Usage example -- given some const FMeshDescription* Mesh:
 *    FMeshDescriptionAABBAdapter MeshAdapter(Mesh); // adapt the mesh
 *    TMeshAABBTree3<const FMeshDescriptionTriangleMeshAdapter> AABBTree(&MeshAdapter); // provide the adapter to a templated class like TMeshAABBTree3
 */
struct /*MESHCONVERSION_API*/ FMeshDescriptionTriangleMeshAdapter
{
	using FIndex3i = UE::Geometry::FIndex3i;
protected:
	const FMeshDescription* Mesh;
	TVertexAttributesConstRef<FVector3f> VertexPositions;
	TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceNormals;
	TVertexInstanceAttributesConstRef<FVector2f> VertexInstanceUVs;
	TArrayView<const FVertexInstanceID> TriangleVertexInstanceIndices;

	FVector3d BuildScale = FVector3d::One();
	bool bScaleNormals = false;

public:
	FMeshDescriptionTriangleMeshAdapter(const FMeshDescription* MeshIn) : Mesh(MeshIn)
	{
		FStaticMeshConstAttributes Attributes(*MeshIn);
		VertexPositions = Attributes.GetVertexPositions();
		VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
		VertexInstanceUVs = Attributes.GetVertexInstanceUVs();
		TriangleVertexInstanceIndices = Attributes.GetTriangleVertexInstanceIndices().GetRawArray();
		// @todo: can we hold TArrayViews of the attribute arrays here? Do we guarantee not to mutate the mesh description for the duration of this object?
	}

	void SetBuildScale(const FVector3d& BuildScaleIn, bool bScaleNormalsIn)
	{
		BuildScale = BuildScaleIn;
		bScaleNormals = bScaleNormalsIn;
	}

	bool IsTriangle(int32 TID) const
	{
		return TID >= 0 && TID < Mesh->Triangles().Num();
	}
	bool IsVertex(int32 VID) const
	{
		return VID >= 0 && VID < Mesh->Vertices().Num();
	}
	// ID and Count are the same for MeshDescription because it's compact
	int32 MaxTriangleID() const
	{
		return Mesh->Triangles().Num();
	}
	int32 TriangleCount() const
	{
		return Mesh->Triangles().Num();
	}
	int32 MaxVertexID() const
	{
		return Mesh->Vertices().Num();
	}
	int32 VertexCount() const
	{
		return Mesh->Vertices().Num();
	}
	uint64 GetChangeStamp() const
	{
		// MeshDescription doesn't provide any mechanism to know if it's been modified so just return 1
		// and leave it to the caller to not build an aabb and then change the underlying mesh
		return 1;
	}
	FIndex3i GetTriangle(int32 IDValue) const
	{
		TArrayView<const FVertexID> TriVertIDs = Mesh->GetTriangleVertices(FTriangleID(IDValue));
		return FIndex3i(TriVertIDs[0].GetValue(), TriVertIDs[1].GetValue(), TriVertIDs[2].GetValue());
	}
	FVector3d GetVertex(int32 IDValue) const
	{
		const FVector3f& Position = VertexPositions[FVertexID(IDValue)];
		return FVector3d(BuildScale.X * (double)Position.X, BuildScale.Y * (double)Position.Y, BuildScale.Z * (double)Position.Z);
	}

	inline void GetTriVertices(int32 IDValue, FVector3d& V0, FVector3d& V1, FVector3d& V2) const
	{
		TArrayView<const FVertexID> TriVertIDs = Mesh->GetTriangleVertices(FTriangleID(IDValue));
		const FVector3f& A = VertexPositions[TriVertIDs[0]];
		V0 = FVector3d(BuildScale.X * (double)A.X, BuildScale.Y * (double)A.Y, BuildScale.Z * (double)A.Z);
		const FVector3f& B = VertexPositions[TriVertIDs[1]];
		V1 = FVector3d(BuildScale.X * (double)B.X, BuildScale.Y * (double)B.Y, BuildScale.Z * (double)B.Z);
		const FVector3f& C = VertexPositions[TriVertIDs[2]];
		V2 = FVector3d(BuildScale.X * (double)C.X, BuildScale.Y * (double)C.Y, BuildScale.Z * (double)C.Z);
	}

	template<typename VectorType>
	inline void GetTriVertices(int32 IDValue, VectorType& V0, VectorType& V1, VectorType& V2) const
	{
		TArrayView<const FVertexID> TriVertIDs = Mesh->GetTriangleVertices(FTriangleID(IDValue));
		const FVector3f& A = VertexPositions[TriVertIDs[0]];
		V0 = VectorType(BuildScale.X * (double)A.X, BuildScale.Y * (double)A.Y, BuildScale.Z * (double)A.Z);
		const FVector3f& B = VertexPositions[TriVertIDs[1]];
		V1 = VectorType(BuildScale.X * (double)B.X, BuildScale.Y * (double)B.Y, BuildScale.Z * (double)B.Z);
		const FVector3f& C = VertexPositions[TriVertIDs[2]];
		V2 = VectorType(BuildScale.X * (double)C.X, BuildScale.Y * (double)C.Y, BuildScale.Z * (double)C.Z);
	}

	inline bool HasNormals() const
	{
		return VertexInstanceNormals.IsValid();
	}
	inline bool IsNormal(int32 NID) const
	{
		return HasNormals() && NID >= 0 && NID < NormalCount();
	}
	inline int32 MaxNormalID() const
	{
		return HasNormals() ? VertexInstanceNormals.GetNumElements() : 0;
	}
	inline int32 NormalCount() const
	{
		return HasNormals() ? VertexInstanceNormals.GetNumElements() : 0;
	}

	/** Get Normal by VertexInstanceID */
	FVector3f GetNormal(int32 IDValue) const
	{
		const FVector3f& InstanceNormal = VertexInstanceNormals[FVertexInstanceID(IDValue)];
		return (!bScaleNormals) ? InstanceNormal :
			UE::Geometry::Normalized(FVector3f(InstanceNormal.X/BuildScale.X, InstanceNormal.Y/BuildScale.Y, InstanceNormal.Z/BuildScale.Z));
	}

	/** Get Normals for a given Triangle */
	template<typename VectorType>
	void GetTriNormals(int32 TriId, VectorType& N0, VectorType& N1, VectorType& N2)
	{
		N0 = GetNormal(TriangleVertexInstanceIndices[TriId*3]);
		N1 = GetNormal(TriangleVertexInstanceIndices[TriId*3+1]);
		N2 = GetNormal(TriangleVertexInstanceIndices[TriId*3+2]);
	}

	inline bool HasUVs(const int32 UVLayer=0) const
	{
		return VertexInstanceUVs.IsValid() && Mesh && UVLayer >= 0 && UVLayer < Mesh->GetNumUVElementChannels();
	}
	inline bool IsUV(const int32 UVId) const
	{
		return HasUVs() && UVId >= 0 && UVId < UVCount();
	}
	inline int32 MaxUVID() const
	{
		return HasUVs() ? VertexInstanceUVs.GetNumElements() : 0;
	}
	inline int32 UVCount() const
	{
		return HasUVs() ? VertexInstanceUVs.GetNumElements() : 0;
	}

	/** Get UV by VertexInstanceID for a given UVLayer */
	FVector2f GetUV(const int32 IDValue, const int32 UVLayer) const
	{
		return VertexInstanceUVs.Get(FVertexInstanceID(IDValue), UVLayer);
	}

	/** Get UVs for a given UVLayer and Triangle */
	template<typename VectorType>
	void GetTriUVs(const int32 TriId, const int32 UVLayer, VectorType& UV0, VectorType& UV1, VectorType& UV2)
	{
		UV0 = GetUV(TriangleVertexInstanceIndices[TriId*3], UVLayer);
		UV1 = GetUV(TriangleVertexInstanceIndices[TriId*3+1], UVLayer);
		UV2 = GetUV(TriangleVertexInstanceIndices[TriId*3+2], UVLayer);
	}
};


/**
 * Non-const version of the adapter, with non-const storage and setters
 * TODO: try to be smarter about sharing code w/ the above const version
 */
struct /*MESHCONVERSION_API*/ FMeshDescriptionEditableTriangleMeshAdapter
{
	using FIndex3i = UE::Geometry::FIndex3i;
protected:
	FMeshDescription* Mesh;
	TVertexAttributesRef<FVector3f> VertexPositions;
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals;

public:
	FMeshDescriptionEditableTriangleMeshAdapter(FMeshDescription* MeshIn) : Mesh(MeshIn)
	{
		FStaticMeshAttributes Attributes(*MeshIn);
		VertexPositions = Attributes.GetVertexPositions();
		VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
	}

	bool IsTriangle(int32 TID) const
	{
		return TID >= 0 && TID < Mesh->Triangles().Num();
	}
	bool IsVertex(int32 VID) const
	{
		return VID >= 0 && VID < Mesh->Vertices().Num();
	}
	// ID and Count are the same for MeshDescription because it's compact
	int32 MaxTriangleID() const
	{
		return Mesh->Triangles().Num();
	}
	int32 TriangleCount() const
	{
		return Mesh->Triangles().Num();
	}
	int32 MaxVertexID() const
	{
		return Mesh->Vertices().Num();
	}
	int32 VertexCount() const
	{
		return Mesh->Vertices().Num();
	}
	uint64 GetChangeStamp() const
	{
		// MeshDescription doesn't provide any mechanism to know if it's been modified so just return 1
		// and leave it to the caller to not build an aabb and then change the underlying mesh
		return 1;
	}
	FIndex3i GetTriangle(int32 IDValue) const
	{
		TArrayView<const FVertexID> TriVertIDs = Mesh->GetTriangleVertices(FTriangleID(IDValue));
		return FIndex3i(TriVertIDs[0].GetValue(), TriVertIDs[1].GetValue(), TriVertIDs[2].GetValue());
	}
	FVector3d GetVertex(int32 IDValue) const
	{
		return FVector3d(VertexPositions[FVertexID(IDValue)]);
	}
	void SetVertex(int32 IDValue, const FVector3d& NewPos)
	{
		VertexPositions[FVertexID(IDValue)] = (FVector3f)NewPos;
	}

	inline void GetTriVertices(int32 IDValue, FVector3d& V0, FVector3d& V1, FVector3d& V2) const
	{
		TArrayView<const FVertexID> TriVertIDs = Mesh->GetTriangleVertices(FTriangleID(IDValue));
		V0 = FVector3d(VertexPositions[TriVertIDs[0]]);
		V1 = FVector3d(VertexPositions[TriVertIDs[1]]);
		V2 = FVector3d(VertexPositions[TriVertIDs[2]]);
	}


	inline bool HasNormals() const
	{
		return VertexInstanceNormals.IsValid();
	}
	inline bool IsNormal(int32 NID) const
	{
		return HasNormals() && NID >= 0 && NID < NormalCount();
	}
	inline int32 MaxNormalID() const
	{
		return HasNormals() ? VertexInstanceNormals.GetNumElements() : 0;
	}
	inline int32 NormalCount() const
	{
		return HasNormals() ? VertexInstanceNormals.GetNumElements() : 0;
	}
	FVector3f GetNormal(int32 IDValue) const
	{
		return FVector3f(VertexInstanceNormals[FVertexInstanceID(IDValue)]);
	}
	void SetNormal(int32 IDValue, const FVector3f& Normal)
	{
		VertexInstanceNormals[FVertexInstanceID(IDValue)] = Normal;
	}
};



/**
 * TTriangleMeshAdapter version of FMeshDescriptionTriangleMeshAdapter
 */
struct FMeshDescriptionMeshAdapterd : public UE::Geometry::TTriangleMeshAdapter<double>
{
	FMeshDescriptionTriangleMeshAdapter ParentAdapter;

	FMeshDescriptionMeshAdapterd(const FMeshDescription* MeshIn) : ParentAdapter(MeshIn)
	{
		IsTriangle = [this](int index) { return ParentAdapter.IsTriangle(index);};
		IsVertex = [this](int index) { return ParentAdapter.IsVertex(index); };
		MaxTriangleID = [this]() { return ParentAdapter.MaxTriangleID();};
		MaxVertexID = [this]() { return ParentAdapter.MaxVertexID();};
		TriangleCount = [this]() { return ParentAdapter.TriangleCount();};
		VertexCount = [this]() { return ParentAdapter.VertexCount();};
		GetChangeStamp = [this]() { return ParentAdapter.GetChangeStamp();};
		GetTriangle = [this](int32 TriangleID) { return ParentAdapter.GetTriangle(TriangleID); };
		GetVertex = [this](int32 VertexID) { return ParentAdapter.GetVertex(VertexID); };
	}

	FMeshDescriptionMeshAdapterd(FMeshDescriptionTriangleMeshAdapter ParentAdapterIn) : ParentAdapter(ParentAdapterIn)
	{
		IsTriangle = [this](int index) { return ParentAdapter.IsTriangle(index);};
		IsVertex = [this](int index) { return ParentAdapter.IsVertex(index); };
		MaxTriangleID = [this]() { return ParentAdapter.MaxTriangleID();};
		MaxVertexID = [this]() { return ParentAdapter.MaxVertexID();};
		TriangleCount = [this]() { return ParentAdapter.TriangleCount();};
		VertexCount = [this]() { return ParentAdapter.VertexCount();};
		GetChangeStamp = [this]() { return ParentAdapter.GetChangeStamp();};
		GetTriangle = [this](int32 TriangleID) { return ParentAdapter.GetTriangle(TriangleID); };
		GetVertex = [this](int32 VertexID) { return ParentAdapter.GetVertex(VertexID); };
	}

};
