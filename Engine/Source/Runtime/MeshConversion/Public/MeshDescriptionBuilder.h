// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "GeometryBase.h"
#include "Math/Box.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"
#include "MeshDescription.h"
#include "MeshTypes.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);
template <typename ElementIDType> class TAttributesSet;


/**
 * These attributes are used to store custom modeling tools data on a MeshDescription
 */
namespace ExtendedMeshAttribute
{
	extern MESHCONVERSION_API const FName PolyTriGroups;
}


/**
 * Utility class to construct MeshDescription instances.
 * NB: this will add a vertex-instance UV to the Description, if it has none. 
 */
class FMeshDescriptionBuilder
{
public:
	MESHCONVERSION_API void SetMeshDescription(FMeshDescription* Description);

	/** Pre-allocate space in the mesh description */
	MESHCONVERSION_API void ReserveNewVertices(int32 Count);

	/** Append vertex and return new vertex ID */
	MESHCONVERSION_API FVertexID AppendVertex(const FVector& Position);

	/** Append vertex with a specific ID and return it */
	MESHCONVERSION_API FVertexID AppendVertexWithId(int32 NewVertexID, const FVector& Position);

	/** Return position of vertex */
	MESHCONVERSION_API FVector GetPosition(const FVertexID& VertexID);

	/** Return position of vertex parent of instance */
	MESHCONVERSION_API FVector GetPosition(const FVertexInstanceID& InstanceID);

	/** Set the position of a vertex */
	MESHCONVERSION_API void SetPosition(const FVertexID& VertexID, const FVector& NewPosition);

	/** Set the number of UV layers */
	MESHCONVERSION_API void SetNumUVLayers(int32 NumUVLayers);

	/** Pre-allocate space in the mesh description for UVs in the indicated UVLayer */
	MESHCONVERSION_API void ReserveNewUVs(int32 Count, int UVLayerIndex);

	/** Append a UV 'vertex' and return a new UV ID*/
	MESHCONVERSION_API FUVID AppendUV(const FVector2D& UVvalue, int32 UVLayerIndex);

	/** Append new vertex instance and return ID */
	MESHCONVERSION_API FVertexInstanceID AppendInstance(const FVertexID& VertexID);

	/** 
	* Set the UV of a vertex instance.
	* Note: this generally shouldn't be called directly because it alters the instance UVs but not the shared UVs
	* When setting UVs use AppendUVTriangle	  
	*/
	MESHCONVERSION_API void SetInstanceUV(const FVertexInstanceID& InstanceID, const FVector2D& InstanceUV, int32 UVLayerIndex = 0);

	/** Set the Normal of a vertex instance*/
	MESHCONVERSION_API void SetInstanceNormal(const FVertexInstanceID& InstanceID, const FVector& Normal);

	/** Set the full tangent space of a vertex instance, this is stored as a vec3 normal, vec3 tangent, and a bitangent sign */
	MESHCONVERSION_API void SetInstanceTangentSpace(const FVertexInstanceID& InstanceID, const FVector& Normal, const FVector& Tangent, float Sign);

	/** Set the Color of a vertex instance*/
	MESHCONVERSION_API void SetInstanceColor(const FVertexInstanceID& InstanceID, const FVector4f& Color);

	/** Enable per-triangle integer attribute named PolyTriGroups */
	MESHCONVERSION_API void EnablePolyGroups();

	/** Set the PolyTriGroups attribute value to a specific GroupID for a Triangle */
	MESHCONVERSION_API void SetPolyGroupID(const FTriangleID& TriangleID, int GroupID);

	/** Append a UV triangle to the specified UV layer. This will use both shared and per-instance UV storage*/
	MESHCONVERSION_API void AppendUVTriangle(const FTriangleID& TriangleID, const FUVID UVverterxID0, const FUVID UVvertexID1, const FUVID UVvertexID2, int32 UVLayerIndex);

	/** Append a triangle to the mesh with the given PolygonGroup ID */
	MESHCONVERSION_API FTriangleID AppendTriangle(const FVertexID& Vertex0, const FVertexID& Vertex1, const FVertexID& Vertex2, const FPolygonGroupID& PolygonGroup);


	/** Append a triangle to the mesh with the given PolygonGroup ID */
	MESHCONVERSION_API FTriangleID AppendTriangle(const FVertexID* Triangle, const FPolygonGroupID& PolygonGroup);


	/**
	 * Append a triangle to the mesh using the given vertex instances and PolygonGroup ID
	 */
	MESHCONVERSION_API FTriangleID AppendTriangle(const FVertexInstanceID& Instance0, const FVertexInstanceID& Instance1, const FVertexInstanceID& Instance2, const FPolygonGroupID& PolygonGroup);

	/**
	 * Append an arbitrary polygon to the mesh with the given PolygonGroup ID
	 * Unique Vertex instances will be created for each polygon-vertex.
	 */
	MESHCONVERSION_API FPolygonID AppendPolygon(const TArray<FVertexID>& Vertices, const FPolygonGroupID& PolygonGroup);


	/** 
	 * Create a new MeshDescription PolygonGroup and return it's ID.
	 * PolygonGroups are not the same as Polygroups, they essentially represent
	 * Mesh Sections, which then reference Material Slots, etc
	 */
	MESHCONVERSION_API FPolygonGroupID AppendPolygonGroup(FName MaterialSlotName = NAME_None);

	/**
	 * Set the specified value for the named attribute / vertex index combination.
	 * If a vertex attribute with the given name doesn't exist, it is created.
	 */
	template<typename T>
	void SetVertexAttributeValue(FName AttributeName, FVertexID VertexID, const T& Value)
	{
		TAttributesSet<FVertexID>& VertexAttributes = MeshDescription->VertexAttributes();

		if (!VertexAttributes.GetAttributesRef<T>(AttributeName).IsValid())
		{
			VertexAttributes.RegisterAttribute<T>(AttributeName);
		}

		VertexAttributes.GetAttributesRef<T>(AttributeName).Set(VertexID, Value);
	}

	/** Set MeshAttribute::Edge::IsHard to true for all edges */
	MESHCONVERSION_API void SetAllEdgesHardness(bool bHard);

	/** Translate the MeshDescription vertex positions */
	MESHCONVERSION_API void Translate(const FVector& Translation);


	/** Return the current bounding box of the mesh */
	MESHCONVERSION_API FBox ComputeBoundingBox() const;

	/** Disable the construction of secondary data structures in the mesh description */
	MESHCONVERSION_API void SuspendMeshDescriptionIndexing();

	/** Enable the construction of secondary data structures in the mesh description */
	MESHCONVERSION_API void ResumeMeshDescriptionIndexing();

protected:

	FMeshDescription* MeshDescription;

	TVertexAttributesRef<FVector3f> VertexPositions;
	TVertexInstanceAttributesRef<FVector2f> InstanceUVs;
	TVertexInstanceAttributesRef<FVector3f> InstanceNormals;
	TVertexInstanceAttributesRef<FVector3f> InstanceTangents;
	TVertexInstanceAttributesRef<float> InstanceBiTangentSign;
	TVertexInstanceAttributesRef<FVector4f> InstanceColors;

	TArray<TUVAttributesRef<FVector2f>> UVCoordinateLayers; 
	TArray<FVertexID> TempBuffer;
	TArray<FUVID> TempUVBuffer;

	TPolygonAttributesRef<int> PolyGroups;

	TPolygonGroupAttributesRef<FName> GroupMaterialSlotNames;
};
