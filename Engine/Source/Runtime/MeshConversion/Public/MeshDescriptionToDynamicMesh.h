// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "GeometryBase.h"
#include "MeshDescription.h"
#include "MeshTypes.h"

// predeclare tangents template
PREDECLARE_GEOMETRY(template<typename RealType> class TMeshTangents);
namespace UE { namespace Geometry { class FDynamicMesh3; } }
struct FMeshDescription;

using UE::Geometry::FDynamicMesh3;

/**
 * Convert FMeshDescription to FDynamicMesh3
 * 
 * @todo handle missing UV/normals on MD?
 * @todo be able to ignore UV/Normals
 * @todo handle additional UV layers on MD
 * @todo option to disable UV/Normal welding
 */
class FMeshDescriptionToDynamicMesh
{
public:
	/** If true, will print some possibly-helpful debugging spew to output log */
	bool bPrintDebugMessages = false;

	/** Should we initialize triangle groups on output mesh */
	bool bEnableOutputGroups = true;

	/** Should we calculate conversion index maps */
	bool bCalculateMaps = true;

	/** Ignore all mesh attributes (e.g. UV/Normal layers, color layer, material groups) */
	bool bDisableAttributes = false;

	/** Should Vertex Colors of MeshDescription be transformed from Linear to SRGB */
	bool bTransformVertexColorsLinearToSRGB = true;

	/** 
	 * Set this flag to transform Polygon Group IDs to their compacted indices, before using them as material indices --
	 * in other words, 'holes' in the Mesh Description's PolygonGroups array are removed by shifting the subsequent IDs down
	 * 
	 * Note: For Static Mesh LODs, this transforms Polygon Group IDs to Section Indices.
	 * This flag should typically be set to 'true' when converting Static Mesh assets.
	 */
	bool bUseCompactedPolygonGroupIDValues = false;

	/**
	 * Set an optional mapping from mesh description polygon group to dynamic mesh material indices; if unset, an identity map is assumed 
	 *
	 * Note: If bUseCompactedPolygonGroupIDValues is true, this remapping applies *after* the compact step --
	 *  so this mapping is a Section Index -> Material Index map for Static Mesh assets.
	 */
	void SetPolygonGroupToMaterialIndexMap(const TArray<int32>& PolygonGroupToMaterialIndexMapIn)
	{
		PolygonGroupToMaterialIndexMap = PolygonGroupToMaterialIndexMapIn;
	}

	/** map from DynamicMesh triangle ID to MeshDescription FTriangleID*/
	TArray<FTriangleID> TriIDMap;

	/**
	* map from DynamicMesh vertex Id to MeshDecription FVertexID. 
	* NB: due to vertex splitting, multiple DynamicMesh vertex ids 
	* may map to the same MeshDescription FVertexID.
	*  ( a vertex split is a result of reconciling non-manifold MeshDescription vertex ) 	  
	*/
	TArray<FVertexID> VertIDMap;


	/**
	* If the source MeshDescription is non-manifold, store mesh vertex id as a vertex attribute on the result mesh 
	* This data can be accessed using FNonManifoldMappingSupport ( see NonManifoldMappingSupport.h)
	* 
	* Note: the vertex attribute will not be created if the source mesh is manifold (as this data would be redundant) or if bDisableAttributes is true.
	*/ 
	bool bVIDsFromNonManifoldMeshDescriptionAttr = false;


	/**
	 * Various modes can be used to create output triangle groups
	 */
	enum class EPrimaryGroupMode
	{
		SetToZero,
		SetToPolygonID,
		SetToPolygonGroupID,
		SetToPolyGroup
	};
	/**
	 * Which mode to use to create groups on output mesh. Ignored if bEnableOutputGroups = false.
	 */
	EPrimaryGroupMode GroupMode = EPrimaryGroupMode::SetToPolyGroup;


	/**
	 * Default conversion of MeshDescription to DynamicMesh
	 * @param bCopyTangents  - if bDisableAttributes is false, this requests the tangent plane vectors (tangent and bitangent) 
	 *                          be stored as overlays in the MeshOut DynamicAttributeSet, provided they exist on the MeshIn
	 */
	MESHCONVERSION_API void Convert(const FMeshDescription* MeshIn, FDynamicMesh3& MeshOut, bool bCopyTangents = false);

	/**
	 * Copy tangents from MeshDescription to a FMeshTangents instance.
	 * @warning Convert() must have been used to create the TargetMesh before calling this function
	 */
	MESHCONVERSION_API void CopyTangents(const FMeshDescription* SourceMesh, const FDynamicMesh3* TargetMesh, UE::Geometry::TMeshTangents<float>* TangentsOut);

	/**
	 * Copy tangents from MeshDescription to a FMeshTangents instance.
	 * @warning Convert() must have been used to create the TargetMesh before calling this function
	 */
	MESHCONVERSION_API void CopyTangents(const FMeshDescription* SourceMesh, const FDynamicMesh3* TargetMesh, UE::Geometry::TMeshTangents<double>* TangentsOut);

protected:
	/**
	 * Applies an optional Linear-to-sRGB color transform on the input. The color transform
	 * is controlled by bTransformVtxColorsLinearToSRGB.
	 *
	 * This is the counterpart to DynamicMeshToMeshDescription::ApplyVertexColorTransform to
	 * undo an applied sRGB-to-Linear transformation when the MeshDescription was built.
	 *
	 * @param Color color to transform
	 */
	MESHCONVERSION_API void ApplyVertexColorTransform(FVector4f& Color) const;

private:

	/** Mapping from mesh description polygon group element IDs to material indices */
	TArray<int32> PolygonGroupToMaterialIndexMap;
};
