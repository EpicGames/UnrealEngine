// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryBase.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"

struct FMeshDescription;

namespace UE {
namespace Geometry {

class FDynamicMesh3;

/**
 * Converter that gives a specific UV layer of a provided mesh description and turns it into a flat
 * FDynamicMesh, with vertices corresponding to UV elements, and their positions corresponding to
 * those elements' UV coordinate values. The triangle id's in the dynamic mesh will correspond to
 * original triangles of the mesh.
 * 
 * The converter can also bake back such a dynamic mesh to an existing mesh description UV layer
 * after the dynamic mesh has been modified. The important part is that the triangle IDs continue
 * to correspond to each other. This means that while welding/splitting vertices in the dynamic
 * mesh is ok, adding or removing triangles is not.
 * 
 * The converter prefers shared UVs when they are available, though it will populate instanced
 * UVs as well on the way back.
 */
class FMeshDescriptionUVsToDynamicMesh
{
public:
	int32 UVLayerIndex = 0;

	// The [0,1] UV range will be scaled to [0, ScaleFactor]
	double ScaleFactor = 1000;

	MESHCONVERSION_API int32 GetNumUVLayers(const FMeshDescription* InputMeshDescription) const;

	// TODO: The conversion function forward to a dynamic mesh should be made to produce maps 
	// similar to MeshDescriptionToDynamicMesh, which is why this is not a const function.
	MESHCONVERSION_API TSharedPtr<FDynamicMesh3> GetUVMesh(const FMeshDescription* InputMeshDescription);

	MESHCONVERSION_API void BakeBackUVsFromUVMesh(const FDynamicMesh3* DynamicUVMesh, FMeshDescription* OutputMeshDescription) const;
};

}}//end namespace UE::Geometry
