// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Bad includes

#include "CoreMinimal.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshResources.h"
#include "TransformTypes.h"


/**
 * Basic struct to adapt a FStaticMeshLODResources for use by GeometryProcessing classes that template the mesh type and expect a standard set of basic accessors
 * For example, this adapter will let you use a FStaticMeshLODResources with GeometryProcessing's TMeshAABBTree3
 */
struct FStaticMeshLODResourcesMeshAdapter
{
	using FIndex3i = UE::Geometry::FIndex3i;
protected:
	const FStaticMeshLODResources* Mesh;

	FVector3d BuildScale = FVector3d::One();
	FVector3d InvBuildScale = FVector3d::One();
	bool bScaleNormals = false;

	TArray<const FStaticMeshSection*> ValidSections;
	TArray<int32> TriangleOffsetArray;
	int32 NumTriangles;

protected:
	FStaticMeshLODResourcesMeshAdapter()
		: Mesh(nullptr)
		, NumTriangles(0)
	{
	}

public:
	FStaticMeshLODResourcesMeshAdapter(const FStaticMeshLODResources* MeshIn)
		: Mesh(MeshIn)
		, NumTriangles(0)
	{
		TriangleOffsetArray.Reserve(MeshIn->Sections.Num() + 1);
		ValidSections.Reserve(MeshIn->Sections.Num());

		for (const FStaticMeshSection& Section : MeshIn->Sections)
		{
			TriangleOffsetArray.Add(NumTriangles);
			NumTriangles += Section.NumTriangles;
			ValidSections.Add(&Section);
		}

		TriangleOffsetArray.Add(NumTriangles);
	}

	void SetBuildScale(const FVector3d& BuildScaleIn, bool bScaleNormalsIn)
	{
		BuildScale = BuildScaleIn;
		InvBuildScale = UE::Geometry::FTransformSRT3d::GetSafeScaleReciprocal(BuildScaleIn);
		bScaleNormals = bScaleNormalsIn;
	}

	bool IsTriangle(int32 TID) const
	{
		return TID >= 0 && TID < MaxTriangleID();
	}
	bool IsVertex(int32 VID) const
	{
		return VID >= 0 && VID < MaxVertexID();
	}
	// ID and Count are the same for StaticMeshLODResources because it's compact
	int32 MaxTriangleID() const
	{
		return TriangleCount();
	}
	int32 TriangleCount() const
	{
		return Mesh && Mesh->IndexBuffer.GetAllowCPUAccess() ? NumTriangles : 0;
	}
	int32 MaxVertexID() const
	{
		return VertexCount();
	}
	int32 VertexCount() const
	{
		return Mesh && Mesh->VertexBuffers.PositionVertexBuffer.GetAllowCPUAccess() ? Mesh->GetNumVertices() : 0;
	}
	uint64 GetChangeStamp() const
	{
		// FStaticMeshLODResource doesn't provide any mechanism to know if it's been modified so just return 1
		// and leave it to the caller to not build an aabb and then change the underlying mesh
		return 1;
	}

	inline const FStaticMeshSection& TriangleToSection(int32& InOutIDValue) const
	{
		int32 SectionIdx = Algo::UpperBound(TriangleOffsetArray, InOutIDValue) - 1;
		InOutIDValue -= TriangleOffsetArray[SectionIdx];
		return *ValidSections[SectionIdx];
	}

	inline FIndex3i GetTriangle(int32 IDValue) const
	{
		const FStaticMeshSection& TriSection = TriangleToSection(IDValue);
		const FIndexArrayView IndexArrayView = Mesh->IndexBuffer.GetArrayView();
		return FIndex3i(IndexArrayView[TriSection.FirstIndex + IDValue * 3 + 0],
			IndexArrayView[TriSection.FirstIndex + IDValue * 3 + 1],
			IndexArrayView[TriSection.FirstIndex + IDValue * 3 + 2]);
	}

	inline FVector3d GetVertex(int32 IDValue) const
	{
		const FVector3f& Position = Mesh->VertexBuffers.PositionVertexBuffer.VertexPosition(IDValue);
		return FVector3d(BuildScale.X * (double)Position.X, BuildScale.Y * (double)Position.Y, BuildScale.Z * (double)Position.Z);
	}

	inline void GetTriVertices(int32 IDValue, FVector3d& V0, FVector3d& V1, FVector3d& V2) const
	{
		FIndex3i VtxIndices = GetTriangle(IDValue);

		V0 = GetVertex(VtxIndices.A);
		V1 = GetVertex(VtxIndices.B);
		V2 = GetVertex(VtxIndices.C);
	}

	template<typename VectorType>
	inline void GetTriVertices(int32 IDValue, VectorType& V0, VectorType& V1, VectorType& V2) const
	{
		FIndex3i VtxIndices = GetTriangle(IDValue);

		V0 = GetVertex(VtxIndices.A);
		V1 = GetVertex(VtxIndices.B);
		V2 = GetVertex(VtxIndices.C);
	}

	inline bool HasNormals() const
	{
		return Mesh && Mesh->VertexBuffers.StaticMeshVertexBuffer.GetAllowCPUAccess() && Mesh->VertexBuffers.StaticMeshVertexBuffer.GetTangentData();
	}
	inline bool IsNormal(int32 NID) const
	{
		return NID >= 0 && NID < NormalCount();
	}
	inline int32 MaxNormalID() const
	{
		return NormalCount();
	}
	inline int32 NormalCount() const
	{
		return HasNormals() ? VertexCount() : 0;
	}

	inline FVector3f GetNormal(int32 IDValue) const
	{
		const FVector4f& Normal = Mesh->VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(IDValue);
		return (!bScaleNormals) ? FVector3f(Normal.X, Normal.Y, Normal.Z) :
			UE::Geometry::Normalized(FVector3f(Normal.X * InvBuildScale.X, Normal.Y * InvBuildScale.Y, Normal.Z * InvBuildScale.Z));
	}

	/** Get Normals for a given Triangle */
	template<typename VectorType>
	inline void GetTriNormals(int32 TriId, VectorType& N0, VectorType& N1, VectorType& N2)
	{
		FIndex3i VtxIndices = GetTriangle(TriId);

		N0 = GetNormal(VtxIndices.A);
		N1 = GetNormal(VtxIndices.B);
		N2 = GetNormal(VtxIndices.C);
	}


	inline FVector3f GetTangentX(int32 IDValue) const
	{
		const FVector4f& TangentX = Mesh->VertexBuffers.StaticMeshVertexBuffer.VertexTangentX(IDValue);
		return (!bScaleNormals) ? FVector3f(TangentX.X, TangentX.Y, TangentX.Z) :
			UE::Geometry::Normalized(FVector3f(TangentX.X * BuildScale.X, TangentX.Y * BuildScale.Y, TangentX.Z * BuildScale.Z));
	}

	/** Get Tangent X for a given Triangle */
	template<typename VectorType>
	inline void GetTriTangentsX(int32 TriId, VectorType& T0, VectorType& T1, VectorType& T2)
	{
		FIndex3i VtxIndices = GetTriangle(TriId);
		T0 = GetTangentX(VtxIndices.A);
		T1 = GetTangentX(VtxIndices.B);
		T2 = GetTangentX(VtxIndices.C);
	}


	inline FVector3f GetTangentY(int32 IDValue) const
	{
		const FVector4f& TangentY = Mesh->VertexBuffers.StaticMeshVertexBuffer.VertexTangentY(IDValue);
		return (!bScaleNormals) ? FVector3f(TangentY.X, TangentY.Y, TangentY.Z) :
			UE::Geometry::Normalized(FVector3f(TangentY.X * BuildScale.X, TangentY.Y * BuildScale.Y, TangentY.Z * BuildScale.Z));
	}

	/** Get Tangent Y for a given Triangle */
	template<typename VectorType>
	inline void GetTriTangentsY(int32 TriId, VectorType& T0, VectorType& T1, VectorType& T2)
	{
		FIndex3i VtxIndices = GetTriangle(TriId);
		T0 = GetTangentY(VtxIndices.A);
		T1 = GetTangentY(VtxIndices.B);
		T2 = GetTangentY(VtxIndices.C);
	}


	inline bool HasUVs(const int32 UVLayer = 0) const
	{
		return Mesh && Mesh->VertexBuffers.StaticMeshVertexBuffer.GetAllowCPUAccess() && Mesh->VertexBuffers.StaticMeshVertexBuffer.GetTexCoordData() && (int32)Mesh->VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords() > UVLayer && UVLayer >= 0;
	}
	inline int32 NumUVLayers() const
	{
		return (Mesh && Mesh->VertexBuffers.StaticMeshVertexBuffer.GetAllowCPUAccess() && Mesh->VertexBuffers.StaticMeshVertexBuffer.GetTexCoordData()) ?
			(int32)Mesh->VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords() : 0;
	}
	inline bool IsUV(const int32 UVId) const
	{
		return HasUVs() && UVId >= 0 && UVId < UVCount();
	}
	inline int32 MaxUVID() const
	{
		return UVCount();
	}
	inline int32 UVCount() const
	{
		return HasUVs() ? Mesh->GetNumVertices() : 0;
	}

	/** Get UV by VertexInstanceID for a given UVLayer */
	inline FVector2f GetUV(const int32 IDValue, const int32 UVLayer) const
	{
		return Mesh->VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(IDValue, UVLayer);
	}

	/** Get UVs for a given UVLayer and Triangle */
	template<typename VectorType>
	inline void GetTriUVs(const int32 TriId, const int32 UVLayer, VectorType& UV0, VectorType& UV1, VectorType& UV2)
	{
		FIndex3i VtxIndices = GetTriangle(TriId);

		UV0 = GetUV(VtxIndices.A, UVLayer);
		UV1 = GetUV(VtxIndices.B, UVLayer);
		UV2 = GetUV(VtxIndices.C, UVLayer);
	}


	inline bool HasColors() const
	{
		return Mesh && Mesh->VertexBuffers.ColorVertexBuffer.GetAllowCPUAccess();
	}
	inline bool IsColor(int32 ColorIndex) const
	{
		return ColorIndex >= 0 && ColorIndex < ColorCount();
	}
	inline int32 MaxColorID() const
	{
		return ColorCount();
	}
	inline int32 ColorCount() const
	{
		return HasColors() ? VertexCount() : 0;
	}

	inline FColor GetColor(int32 IDValue) const
	{
		return Mesh->VertexBuffers.ColorVertexBuffer.VertexColor(IDValue);
	}

	/** Get Colors for a given Triangle */
	inline void GetTriColors(int32 TriId, FColor& C0, FColor& C1, FColor& C2)
	{
		FIndex3i VtxIndices = GetTriangle(TriId);
		C0 = GetColor(VtxIndices.A);
		C1 = GetColor(VtxIndices.B);
		C2 = GetColor(VtxIndices.C);
	}

};