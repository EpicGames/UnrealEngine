// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMeshOverlay.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "DynamicMesh/DynamicBoneAttribute.h"
#include "VectorTypes.h"
#include "Async/Async.h"

namespace UE
{
namespace Geometry
{ 


/**
*
* Class used to convert a mesh without attributes (e.g. normals, uvs etc) to a FDynamicMesh3
*
* The Source Mesh has to implement this interface
*
*	struct FSrcMeshInterface
*	{
*       // ID types:  must be castable to int32
*		typedef SrcVertIDType    VertIDType;
*		typedef SrcTriIDType     TriIDType;
*
*		// accounting.
*		int32 NumTris() const;
*		int32 NumVerts() const;
*
*	    // --"Vertex Buffer" info
*		const Iterable_VertIDType& GetVertIDs() const;
*		const FVector GetPosition(const VertIDType VtxID) const;
*
*       // --"Index Buffer" info
*		const Iterable_TriIDType& GetTriIDs() const
*		// return false if this TriID is not contained in mesh.
*		bool GetTri(const TriIDType TriID, VertIDType& VID0, VertIDType& VID1, VertIDType& VID2) const;
*	};
*
*/
template<typename SrcMeshType>
class TToDynamicMeshBase
{
public:

	using SrcVertIDType = typename SrcMeshType::VertIDType;
	using SrcTriIDType = typename SrcMeshType::TriIDType;


	TToDynamicMeshBase()
	{
	}


	// Mapping data captured by conversion
	TArray<SrcVertIDType> ToSrcVertIDMap;
	TArray<SrcTriIDType>  ToSrcTriIDMap;

	FDateTime Time_AfterVertices;
	FDateTime Time_AfterTriangles;

	// Produces a DynamicMesh3 w/o attributes from the MeshIn.
	void Convert(FDynamicMesh3& MeshOut, const SrcMeshType& MeshIn, TFunctionRef<int32(const SrcTriIDType& SrcTrID)> GroupIDFunction)
	{
		MeshOut.Clear();
		MeshOut.DiscardAttributes();

		MeshOut.EnableTriangleGroups(0);

		ToSrcVertIDMap.Reset();
		ToSrcTriIDMap.Reset();

		const int32 NumSrcVerts = MeshIn.NumVerts();
		const int32 NumSrcTris = MeshIn.NumTris();

		if (NumSrcVerts == 0 || NumSrcTris == 0)
		{
			return;
		}
		// allocate for map data
		
		ToSrcVertIDMap.AddUninitialized(NumSrcVerts);
		ToSrcTriIDMap.AddUninitialized(NumSrcTris);
		// copy vertex positions. Later we may have to append duplicate vertices to resolve non-manifold structures.
		int32 MaxSrcVertID = -1;
		for (const SrcVertIDType& SrcVertID : MeshIn.GetVertIDs())
		{
			const FVector3d Position = MeshIn.GetPosition(SrcVertID);
			const int32 DstVertID = MeshOut.AppendVertex(Position);
			ToSrcVertIDMap[DstVertID] = SrcVertID;

			MaxSrcVertID = FMath::Max((int32)SrcVertID, MaxSrcVertID);
		}
		// map the other direction
		TArray<int32> FromSrcVertIDMap;
		FromSrcVertIDMap.AddUninitialized(MaxSrcVertID+1);
		for (int32 i = 0; i < ToSrcVertIDMap.Num(); ++i)
		{
			int32 SrcVertID = (int32)ToSrcVertIDMap[i];
			FromSrcVertIDMap[SrcVertID] = i;
		}

		Time_AfterVertices = FDateTime::Now();

		// copy the index buffer.  Note, this may add vertices if a non-manifold state has to be resolved.
		for (const SrcTriIDType& SrcTriID : MeshIn.GetTriIDs())
		{

			const int32 DstGroupID = GroupIDFunction(SrcTriID);

			SrcVertIDType SrcTriVerts[3];
			MeshIn.GetTri(SrcTriID, SrcTriVerts[0], SrcTriVerts[1], SrcTriVerts[2]);
	
			FIndex3i DstTriVerts(FromSrcVertIDMap[SrcTriVerts[0]],
				                 FromSrcVertIDMap[SrcTriVerts[1]],
				                 FromSrcVertIDMap[SrcTriVerts[2]]);

			// attemp to add tri. this may fail if DynamicMesh is more topologically restrictive than the src mesh.
			int32 DstTriangleID = MeshOut.AppendTriangle(DstTriVerts, DstGroupID);

			//-- already seen this triangle for some reason.. or the src mesh had a degenerate tri
			if (DstTriangleID == FDynamicMesh3::DuplicateTriangleID || DstTriangleID == FDynamicMesh3::InvalidID)
			{
				continue;
			}

			//-- non manifold 
			// if append failed due to non-manifold, duplicate verts
			if (DstTriangleID == FDynamicMesh3::NonManifoldID)
			{
				// determine which verts need to be duplicated
				bool bDuplicate[3] = { false, false, false };
				for (int i = 0; i < 3; ++i)
				{
					int ii = (i + 1) % 3;
					int EID = MeshOut.FindEdge(DstTriVerts[i], DstTriVerts[ii]);
					if (EID != FDynamicMesh3::InvalidID && MeshOut.IsBoundaryEdge(EID) == false)
					{
						bDuplicate[i] = true;
						bDuplicate[ii] = true;
					}
				}
				for (int i = 0; i < 3; ++i)
				{
					if (bDuplicate[i])
					{
						const FVector3d Position = MeshOut.GetVertex(DstTriVerts[i]);
						const int32 NewDstVertID = MeshOut.AppendVertex(Position);
						DstTriVerts[i] = NewDstVertID;
						// grow the map to make room
						ToSrcVertIDMap.SetNumUninitialized(NewDstVertID + 1);

						ToSrcVertIDMap[NewDstVertID] = SrcTriVerts[i];
					}
				}

				// add the fixed tri
				DstTriangleID = MeshOut.AppendTriangle(DstTriVerts, DstGroupID);
				checkSlow(DstTriangleID != FDynamicMesh3::NonManifoldID);
			}

			// record in map
			ToSrcTriIDMap[DstTriangleID] = SrcTriID;
		}

		const int32 MaxDstTriID = MeshOut.MaxTriangleID(); // really MaxTriID+1
		// if the source mesh had duplicates then TriIDMap will be too long, this will trim excess
		ToSrcTriIDMap.SetNum(MaxDstTriID);
		Time_AfterTriangles = FDateTime::Now();
	}
};


// Used for exact attribute value welding.
template <typename AttrType>
struct TVertexAttr
{
	int VID;
	AttrType AttrValue;
	bool operator==(const TVertexAttr& o) const
	{
		return VID == o.VID && AttrValue == o.AttrValue;
	}
};

template <typename AttrType>
inline uint32 GetTypeHash(const TVertexAttr<AttrType>& Vector)
{
	// ugh copied from FVector clearly should not be using CRC for hash!!
	return FCrc::MemCrc32(&Vector, sizeof(Vector));
}

template <typename AttrType> struct TOverlayTraits {};
template<> struct TOverlayTraits<FVector3f> { typedef FDynamicMeshNormalOverlay OverlayType; };
template<> struct TOverlayTraits<FVector2f> { typedef FDynamicMeshUVOverlay OverlayType; };
template<> struct TOverlayTraits<FVector4f> { typedef FDynamicMeshColorOverlay OverlayType; };

// Welder used for exact attribute value welding when constructing overlay
template <typename AttrType>
class TAttrWelder
{
public:
	using OverlayType = typename TOverlayTraits<AttrType>::OverlayType;
	using FVertexAttr = TVertexAttr<AttrType>;

	TMap<FVertexAttr, int> UniqueVertexAttrs;
	OverlayType* Overlay;

	TAttrWelder() : Overlay(nullptr){}

	TAttrWelder(OverlayType* OverlayIn)
	{
		check(OverlayIn);
		Overlay = OverlayIn;
	}

	int FindOrAddUnique(const AttrType& AttrValue, int VertexID)
	{
		FVertexAttr VertAttr = { VertexID, AttrValue };

		const int32* FoundIndex = UniqueVertexAttrs.Find(VertAttr);
		if (FoundIndex != nullptr)
		{
			return *FoundIndex;
		}

		int32 NewIndex = Overlay->AppendElement(AttrValue);
		UniqueVertexAttrs.Add(VertAttr, NewIndex);
		return NewIndex;
	}
};

/**
*
* Class used to convert a mesh with attributes (e.g. normals, uvs etc) to a FDynamicMesh3
*
* The Source Mesh has to implement this interface
*
*	struct FSrcMeshInterface
*	{
*       // ID types:  must be castable to int32
*		typedef SrcVertIDType    VertIDType;
*		typedef SrcTriIDType     TriIDType;
*		typedef SrcUVIDType      UVIDType;
*		typedef SrcNormalIDType  NormalIDType;
*		typedef SrcColorIDType   ColorIDType
*       typedef SrcWedgeIDType   WedgeIDType;
*
*		// accounting.
*		int32 NumTris() const;
*		int32 NumVerts() const;
*       int32 NumUVLayers() const;
* 
*	    // --"Vertex Buffer" info 
*		const Iterable_VertIDType& GetVertIDs() const;
*		const FVector3d GetPosition(const VertIDType VtxID) const;
*
*       // --"Index Buffer" info
*		const Iterable_TriIDType& GetTriIDs() const
*		// return false if this TriID is not contained in mesh.
*		bool GetTri(const TriIDType TriID, VertIDType& VID0, VertIDType& VID1, VertIDType& VID2) const;
* 
*		bool HasNormals() const;
*		bool HasTangents() cosnt;
*		bool HasBiTangents() const;
*	
*       // Each triangle corner is a wedge
*       void GetWedgeID(const TriIDType& TriID, WedgeIDType& WID0, WedgeIDType& WID1, WedgeIDType& WID2) const;
*		
*    	// attribute access per-wedge
*		// NB:  ToDynamicMesh will attempt to weld identical attributes that are associated with the same vertex
*		FVector2f GetWedgeUV(int32 UVLayerIndex, WedgeIDType WID) const;
*		FVector3f GetWedgeNormal(WedgeIDType WID) const;
*		FVector3f GetWedgeTangent(WedgeIDType WID) const;
*		FVector3f GetWedgeBiTangent(WedgeIDType WID) const;
*       FVector4f GetWedgeColor(WedgeIDType WID) const;
* 
*		// attribute access that exploits shared attributes. 
*		// each group of shared attributes presents itself as a mesh with its own attribute vertex buffer.
*		// NB:  If the mesh has no shared Attr attributes, then Get{Attr}IDs() should return an empty array.
*       // NB:  Get{Attr}Tri() functions should return false if the triangle is not set in the attribute mesh. 
*       
*       const TArray<UVIDType>& GetUVIDs(int32 LayerID) const;
*		FVector2f GetUV(int32 LayerID, UVIDType UVID) const;
*		bool GetUVTri(int32 LayerID, const TriIDType& TID, UVIDType& ID0, UVIDType& ID1, UVIDType& ID2) const;
*
*       const TArray<NormalIDType>& GetNormalIDs() const;
*		FVector3f GetNormal(NormalIDType ID) const;
*		bool GetNormalTri(const TriIDType& TID, NormalIDType& NID0, NormalIDType& NID1, NormalIDType& NID2) const;
*
*		const TArray<NormalIDType>& GetTangentIDs() const;
* 		FVector3f GetTangent(NormalIDType ID) const;
*		bool GetTangentTri(const TriIDType& TID, NormalIDType& NID0, NormalIDType& NID1, NormalIDType& NID2) const;
* 
* 		const TArray<NormalIDType>& GetBiTangentIDs() const;
*		FVector3f GetBiTangent(NormalIDType ID) const;
*       bool GetBiTangentTri(const TriIDType& TID, NormalIDType& NID0, NormalIDType& NID1, NormalIDType& NID2) const;
* 
*		const TArray<ColorIDType>& GetColorIDs() const;
*		FVector4f GetColor(ColorIDType ID) const;
*		bool GetColorTri(const TriIDType& TID, ColorIDType& NID0, ColorIDType& NID1, ColorIDType& NID2) const;
* 
*       // weight maps information
*	    int32 NumWeightMapLayers() const;
*       float GetVertexWeight(int32 WeightMapIndex, int32 SrcVertID) const;
*	    FName GetWeightMapName(int32 WeightMapIndex) const;
*
*		// skin weight attributes information
*		int32 NumSkinWeightAttributes() const;
*		UE::AnimationCore::FBoneWeights GetVertexSkinWeight(int32 SkinWeightAttributeIndex, VertIDType VtxID) const;
*		FName GetSkinWeightAttributeName(int32 SkinWeightAttributeIndex) const;
*
*		// bone attributes information
*		int32 GetNumBones() const;
*		FName GetBoneName(int32 BoneIdx) const;
*		int32 GetBoneParentIndex(int32 BoneIdx) const;
*		FTransform GetBonePose(int32 BoneIdx) const;
*		FVector4f GetBoneColor(int32 BoneIdx) const;
*	};
*
*/
template <typename SrcMeshType>
class TToDynamicMesh : public TToDynamicMeshBase<SrcMeshType>
{
public:
	typedef TToDynamicMeshBase<SrcMeshType>  MyBase;
	using SrcTriIDType    = typename MyBase::SrcTriIDType;
	using SrcVertIDType   = typename MyBase::SrcVertIDType;
	using SrcUVIDType     = typename SrcMeshType::UVIDType;
	using SrcNormalIDType = typename SrcMeshType::NormalIDType;
	using SrcColorIDType  = typename SrcMeshType::ColorIDType;
	using SrcWedgeIDType  = typename SrcMeshType::WedgeIDType;

	TToDynamicMesh() :MyBase() {}



	// Convert To FDynamicMesh w/o attributes
	void ConvertWOAttributes(FDynamicMesh3& MeshOut,
		const SrcMeshType& MeshIn,
		TFunctionRef<int32(const SrcTriIDType& SrcTrID)> GroupIDFunction)
	{
		MyBase::Convert(MeshOut, MeshIn, GroupIDFunction);
	}


	// Convert To FDynamicMesh.  Will Copy GroupID to the mesh, and will create overlays with UVs, Normal, MaterialID (additionally Tangents and BiTangents if requested)
	void Convert(FDynamicMesh3& MeshOut,
		const SrcMeshType& MeshIn,
		TFunctionRef<int32(const SrcTriIDType& SrcTrID)> GroupIDFunction,
		TFunctionRef<int32(const SrcTriIDType& SrcTrID)> MaterialIDFunction,
		bool bCopyTangents)
	{

		MyBase::Convert(MeshOut, MeshIn, GroupIDFunction); // convert the mesh.  Must call before populate overlays
		PopulateOverlays(MeshOut, MeshIn, MaterialIDFunction, bCopyTangents); // convert the attributes
	}

protected:


	// Populates overlays for UVs, Normal, Color, Material ID. Also Tangent and BiTangent if bCopyTangents == true.  
	// NB: This does not copy any polygroup information.  
	void PopulateOverlays(FDynamicMesh3& MeshOut, const SrcMeshType& MeshIn, TFunctionRef<int32(const SrcTriIDType& SrcTrID)> MaterialIDFunction, bool bCopyTangents)
	{
		// Attributes we want to transfer
		FDynamicMeshNormalOverlay* NormalOverlay = nullptr;
		FDynamicMeshNormalOverlay* TangentOverlay = nullptr;
		FDynamicMeshNormalOverlay* BiTangentOverlay = nullptr;
		FDynamicMeshColorOverlay*  ColorOverlay = nullptr;
		FDynamicMeshMaterialAttribute* MaterialIDAttrib = nullptr;

		// only copy tangents if they exist
		bCopyTangents = bCopyTangents && MeshIn.HasTangents() && MeshIn.HasBiTangents();

		// -- Create Overlays
		const int32 NumUVLayers = MeshIn.NumUVLayers();

		MeshOut.EnableAttributes(); // by default 1-UV layer and 1-normal layer

		// create any addition Normal overlays and reserve space.
		int32 NumRequiredNormalLayers = (bCopyTangents) ? 3 : 1;
		if (MeshOut.Attributes()->NumNormalLayers() < NumRequiredNormalLayers)
		{
			MeshOut.Attributes()->SetNumNormalLayers(NumRequiredNormalLayers);
		}
		for (int32 i = 0; i < NumRequiredNormalLayers; ++i)
		{
			MeshOut.Attributes()->GetNormalLayer(i)->InitializeTriangles(MeshOut.MaxTriangleID());
		}

		NormalOverlay = MeshOut.Attributes()->PrimaryNormals();
		if (bCopyTangents)
		{
			TangentOverlay   = MeshOut.Attributes()->PrimaryTangents();
			BiTangentOverlay = MeshOut.Attributes()->PrimaryBiTangents();
		}

		// create UV overlays 
		MeshOut.Attributes()->SetNumUVLayers(FMath::Max(1, NumUVLayers));
		// reserve space in any new UV layers.
		for (int32 i = 1; i < NumUVLayers; ++i)
		{
			MeshOut.Attributes()->GetUVLayer(i)->InitializeTriangles(MeshOut.MaxTriangleID());
		}

		// create color overlay
		if (MeshIn.HasColors())
		{
			MeshOut.Attributes()->EnablePrimaryColors();
			ColorOverlay = MeshOut.Attributes()->PrimaryColors();
			ColorOverlay->InitializeTriangles(MeshOut.MaxTriangleID());
		}

		// always enable Material ID if there are any attributes
		MeshOut.Attributes()->EnableMaterialID();
		MaterialIDAttrib = MeshOut.Attributes()->GetMaterialID();

		// do the conversions.
		// we will populate all the attributes simultaneously, hold on to futures in this array and then Wait for them at the end
		TArray<TFuture<void>> Pending;



		// populate UV overlays
		for (int UVLayerIndex = 0; UVLayerIndex < NumUVLayers; UVLayerIndex++)
		{
			auto UVFuture = Async(EAsyncExecution::ThreadPool, [&, UVLayerIndex]()
				{
					FDynamicMeshUVOverlay* Overlay = MeshOut.Attributes()->GetUVLayer(UVLayerIndex);

					PopulateOverlay<FVector2f, SrcUVIDType>(
						Overlay,
						MeshIn.GetUVIDs(UVLayerIndex),
						[&MeshIn, UVLayerIndex](const SrcUVIDType& UVID)->FVector2f { return MeshIn.GetUV(UVLayerIndex, UVID); },
						[&MeshIn, UVLayerIndex](const SrcTriIDType& TriID, SrcUVIDType& UV0, SrcUVIDType& UV1, SrcUVIDType& UV2)->bool {return MeshIn.GetUVTri(UVLayerIndex, TriID, UV0, UV1, UV2); },
						[&MeshIn, UVLayerIndex](const SrcWedgeIDType& WID)->FVector2f {return MeshIn.GetWedgeUV(UVLayerIndex, WID); },
						[&MeshIn](const SrcTriIDType& TriID, SrcWedgeIDType& WID0, SrcWedgeIDType& WID1, SrcWedgeIDType& WID2)->void { MeshIn.GetWedgeIDs(TriID, WID0, WID1, WID2); }
					);
				});
			Pending.Add(MoveTemp(UVFuture));
		}
		// populate Normal overlay
		if (NormalOverlay != nullptr)
		{
			auto NormalFuture = Async(EAsyncExecution::ThreadPool, [&]()
				{
					PopulateOverlay<FVector3f, SrcNormalIDType>(
						NormalOverlay,
						MeshIn.GetNormalIDs(),
						[&MeshIn](const SrcNormalIDType& NID)->FVector3f { return MeshIn.GetNormal(NID); },
						[&MeshIn](const SrcTriIDType& TriID, SrcNormalIDType& N0, SrcNormalIDType& N1, SrcNormalIDType& N2)->bool {return MeshIn.GetNormalTri(TriID, N0, N1, N2); },
						[&MeshIn](const SrcWedgeIDType& WID)->FVector3f {return MeshIn.GetWedgeNormal(WID); },
						[&MeshIn](const SrcTriIDType& TriID, SrcWedgeIDType& WID0, SrcWedgeIDType& WID1, SrcWedgeIDType& WID2)->void { MeshIn.GetWedgeIDs(TriID, WID0, WID1, WID2); }
					);
				});
			Pending.Add(MoveTemp(NormalFuture));
		}
		// populate Tangent overlay
		if (TangentOverlay != nullptr)
		{
			auto TangentFuture = Async(EAsyncExecution::ThreadPool, [&]()
				{
					PopulateOverlay<FVector3f, SrcNormalIDType>(
						TangentOverlay,
						MeshIn.GetTangentIDs(),
						[&MeshIn](const SrcNormalIDType& NID)->FVector3f { return MeshIn.GetTangent(NID); },
						[&MeshIn](const SrcTriIDType& TriID, SrcNormalIDType& N0, SrcNormalIDType& N1, SrcNormalIDType& N2)->bool {return MeshIn.GetTangentTri(TriID, N0, N1, N2); },
						[&MeshIn](const SrcWedgeIDType& WID)->FVector3f {return MeshIn.GetWedgeTangent(WID); },
						[&MeshIn](const SrcTriIDType& TriID, SrcWedgeIDType& WID0, SrcWedgeIDType& WID1, SrcWedgeIDType& WID2)->void { MeshIn.GetWedgeIDs(TriID, WID0, WID1, WID2); }
					);
				});
			Pending.Add(MoveTemp(TangentFuture));
		}
		//populate BiTangent overlay
		if (BiTangentOverlay != nullptr)
		{
			auto BiTangentFuture = Async(EAsyncExecution::ThreadPool, [&]()
				{
					PopulateOverlay<FVector3f, SrcNormalIDType>(
						BiTangentOverlay,
						MeshIn.GetBiTangentIDs(),
						[&MeshIn](const SrcNormalIDType& NID)->FVector3f { return MeshIn.GetBiTangent(NID); },
						[&MeshIn](const SrcTriIDType& TriID, SrcNormalIDType& N0, SrcNormalIDType& N1, SrcNormalIDType& N2)->bool {return MeshIn.GetBiTangentTri(TriID, N0, N1, N2); },
						[&MeshIn](const SrcWedgeIDType& WID)->FVector3f {return MeshIn.GetWedgeBiTangent(WID); },
						[&MeshIn](const SrcTriIDType& TriID, SrcWedgeIDType& WID0, SrcWedgeIDType& WID1, SrcWedgeIDType& WID2)->void { MeshIn.GetWedgeIDs(TriID, WID0, WID1, WID2); }
					);

				});
			Pending.Add(MoveTemp(BiTangentFuture));
		}
		if (ColorOverlay != nullptr)
		{
			auto ColorFuture = Async(EAsyncExecution::ThreadPool, [&]()
				{
					PopulateOverlay<FVector4f, SrcColorIDType>(
						ColorOverlay,
						MeshIn.GetColorIDs(),
						[&MeshIn](const SrcColorIDType& CID)->FVector4f { return MeshIn.GetColor(CID); },
						[&MeshIn](const SrcTriIDType& TriID, SrcColorIDType& C0, SrcColorIDType& C1, SrcColorIDType& C2)->bool {return MeshIn.GetColorTri(TriID, C0, C1, C2); },
						[&MeshIn](const SrcWedgeIDType& WID)->FVector4f {return MeshIn.GetWedgeColor(WID); },
						[&MeshIn](const SrcTriIDType& TriID, SrcWedgeIDType& WID0, SrcWedgeIDType& WID1, SrcWedgeIDType& WID2)->void { MeshIn.GetWedgeIDs(TriID, WID0, WID1, WID2); }
					);

				});
			Pending.Add(MoveTemp(ColorFuture));
		}

		//populate MaterialID overlay
		if (MaterialIDAttrib != nullptr)
		{
			auto MaterialFuture = Async(EAsyncExecution::ThreadPool, [&]()
				{
					for (int32 TriangleID : MeshOut.TriangleIndicesItr())
					{
						SrcTriIDType SrcTriID = MyBase::ToSrcTriIDMap[TriangleID];
						MaterialIDAttrib->SetValue(TriangleID, MaterialIDFunction(SrcTriID));
					}
				});
			Pending.Add(MoveTemp(MaterialFuture));
		}

		//populate WeightMap attribute
		const int32 NumWeightMaps = MeshIn.NumWeightMapLayers();
		MeshOut.Attributes()->SetNumWeightLayers(NumWeightMaps);
		for (int WeightMapIndex = 0; WeightMapIndex < NumWeightMaps; WeightMapIndex++)
		{
			auto WeightMapFuture = Async(EAsyncExecution::ThreadPool, [this, &MeshIn, &MeshOut, WeightMapIndex]()
			{
				FDynamicMeshWeightAttribute* WeightMapAttrib = MeshOut.Attributes()->GetWeightLayer(WeightMapIndex);

				for (int32 VertexID : MeshOut.VertexIndicesItr())
				{
					SrcVertIDType SrcVertID = MyBase::ToSrcVertIDMap[VertexID];
					float Weight = MeshIn.GetVertexWeight(WeightMapIndex, SrcVertID);
					WeightMapAttrib->SetValue(VertexID, &Weight);
				}

				WeightMapAttrib->SetName( MeshIn.GetWeightMapName(WeightMapIndex) );
			});
			Pending.Add(MoveTemp(WeightMapFuture));
		}

		
		// populate skinning weight attribute
		const int32 NumSkinWeightAttributes = MeshIn.NumSkinWeightAttributes();
		
		// first, attach all the skinning weight attributes (this also allocates memory to store skinning weights for each attribute)
		for (int AttributeIndex = 0; AttributeIndex < NumSkinWeightAttributes; ++AttributeIndex)
		{
			UE::Geometry::FDynamicMeshVertexSkinWeightsAttribute* Attribute = new UE::Geometry::FDynamicMeshVertexSkinWeightsAttribute(&MeshOut);
			const FName AttribName = MeshIn.GetSkinWeightAttributeName(AttributeIndex);
			Attribute->SetName(AttribName);
			MeshOut.Attributes()->AttachSkinWeightsAttribute(AttribName, Attribute);
		}

		// now set all of the skin weight data
		for (int AttributeIndex = 0; AttributeIndex < NumSkinWeightAttributes; ++AttributeIndex)
		{
			auto SkinAttributeFeature = Async(EAsyncExecution::ThreadPool, [this, &MeshIn, &MeshOut, AttributeIndex]()
			{
				const FName AttribName = MeshIn.GetSkinWeightAttributeName(AttributeIndex);
				
				UE::Geometry::FDynamicMeshVertexSkinWeightsAttribute* OutAttribute = MeshOut.Attributes()->GetSkinWeightsAttribute(AttribName);
				checkSlow(OutAttribute != nullptr);

				if (OutAttribute) 
				{
					for (int32 VertexID : MeshOut.VertexIndicesItr())
					{
						SrcVertIDType SrcVertID = MyBase::ToSrcVertIDMap[VertexID];
						UE::AnimationCore::FBoneWeights SkinWeight = MeshIn.GetVertexSkinWeight(AttributeIndex, SrcVertID);
						OutAttribute->SetValue(VertexID, SkinWeight);
					}
				}
			});
			Pending.Add(MoveTemp(SkinAttributeFeature));
		}

		// populate bones attribute
		const int NumBones = MeshIn.GetNumBones();
		if (MeshIn.GetNumBones())
		{	
			MeshOut.Attributes()->EnableBones(NumBones);
			FDynamicMeshBoneNameAttribute* BoneNameAttrib = MeshOut.Attributes()->GetBoneNames();
			FDynamicMeshBoneParentIndexAttribute* BoneParentIndices = MeshOut.Attributes()->GetBoneParentIndices();
			FDynamicMeshBonePoseAttribute* BonePoses = MeshOut.Attributes()->GetBonePoses();
			FDynamicMeshBoneColorAttribute* BoneColors = MeshOut.Attributes()->GetBoneColors();

			auto BoneAttributeFeature = Async(EAsyncExecution::ThreadPool, [this, &MeshIn, &MeshOut, BoneNameAttrib, BoneParentIndices, BonePoses, BoneColors, NumBones]()
			{
				for (int32 BoneIdx = 0; BoneIdx < NumBones; ++BoneIdx)
				{
					BoneNameAttrib->SetValue(BoneIdx, MeshIn.GetBoneName(BoneIdx));
					BoneParentIndices->SetValue(BoneIdx, MeshIn.GetBoneParentIndex(BoneIdx));
					BonePoses->SetValue(BoneIdx, MeshIn.GetBonePose(BoneIdx));
					BoneColors->SetValue(BoneIdx, MeshIn.GetBoneColor(BoneIdx));
				}
			});
			Pending.Add(MoveTemp(BoneAttributeFeature));
		}

		// TODO: PolyGroup layers

		// wait for all work to be done
		for (TFuture<void>& Future : Pending)
		{
			Future.Wait();
		}
	}


	// Access the attributes in the Source Mesh on a per-wedge granularity (each corner of each triangle is a wedge), and weld them based on strict equality when creating the overlay.
	template<typename AttrType>
	void PopulateOverlayFromWedgeAttr(typename TOverlayTraits<AttrType>::OverlayType* Overlay,
		TFunctionRef<AttrType(const SrcWedgeIDType& WedgeID)> WedgeAttrs,
		TFunctionRef<void(const SrcTriIDType& SrcTriID, SrcWedgeIDType& WID0, SrcWedgeIDType& WID1, SrcWedgeIDType& WID2)> TriToWedges,
		bool bWeldIdenticalAttrs = true)
	{

		FDynamicMesh3& MeshOut = *Overlay->GetParentMesh();
		TAttrWelder<AttrType> AttrWelder(Overlay);

		for (int32 TriangleID : MeshOut.TriangleIndicesItr())
		{
			// skip overlay triangles that a prior method has populated.
			// NB: this function may have been called after PopulateOverlayFromSharedAttr
			if (Overlay->IsSetTriangle(TriangleID))
			{
				continue;
			}

			const SrcTriIDType& SrcTriID = MyBase::ToSrcTriIDMap[TriangleID];

			SrcWedgeIDType WID[3];
			TriToWedges(SrcTriID, WID[0], WID[1], WID[2]);

			FIndex3i Tri = MeshOut.GetTriangle(TriangleID);
			FIndex3i AttrTri;
			for (int j = 0; j < 3; ++j)
			{
				const AttrType& AttrValue = WedgeAttrs(WID[j]);
				if (bWeldIdenticalAttrs)
				{
					AttrTri[j] = AttrWelder.FindOrAddUnique(AttrValue, Tri[j]);
				}
				else
				{
					AttrTri[j] = Overlay->AppendElement(AttrValue);
				}
			}
			Overlay->SetTriangle(TriangleID, AttrTri);
		}
	}


	template <typename AttrType, typename SrcAttrIDType>
	void PopulateOverlayFromSharedAttr(typename TOverlayTraits<AttrType>::OverlayType* Overlay,
		const TArray<SrcAttrIDType>& SrcAttrIDs,
		TFunctionRef<AttrType(const SrcAttrIDType& AttrID)>  SharedAttrValues,
		TFunctionRef<bool(const SrcTriIDType& SrcTriID, SrcAttrIDType& AttrID0, SrcAttrIDType& AttrID1, SrcAttrIDType& AttrID2)> AttrTriIndices)
	{
		FDynamicMesh3& MeshOut = *Overlay->GetParentMesh();

		// return if no shared attributes.
		if (SrcAttrIDs.Num() == 0)
		{
			return;
		}

		
		int32 MaxSrcAttrID = -1;
		for (const SrcAttrIDType& SrcAttrID : SrcAttrIDs)
		{
			MaxSrcAttrID = FMath::Max(MaxSrcAttrID, (int32)SrcAttrID);
		}
		TArray<int32> FromSrcAttrIDMap;
		FromSrcAttrIDMap.Init(-1, MaxSrcAttrID+1);
		// copy attr values into the overlay - these are vertices in the attr mesh.
		for (const SrcAttrIDType& SrcAttrID : SrcAttrIDs)
		{
			const AttrType AttrValue = SharedAttrValues(SrcAttrID);
			const int32 NewIndex = Overlay->AppendElement(AttrValue);
			int32 SrcAttrID32 = (int32)SrcAttrID;
			FromSrcAttrIDMap[SrcAttrID32] = NewIndex;
		}

		// use the attr index buffer to make triangles in the overlay
		for (int32 TriID : MeshOut.TriangleIndicesItr())
		{
			const SrcTriIDType SrcTriID = MyBase::ToSrcTriIDMap[TriID];
			SrcAttrIDType SrcAttrTri[3];
			bool bHasValidAttrTri = AttrTriIndices(SrcTriID, SrcAttrTri[0], SrcAttrTri[1], SrcAttrTri[2]);

			if (!bHasValidAttrTri)
			{
				// don't set this tri in the overlay
				continue;
			}

			// translate to Overlay indicies 
			FIndex3i AttrTri(FromSrcAttrIDMap[(int32)SrcAttrTri[0]],
				FromSrcAttrIDMap[(int32)SrcAttrTri[1]],
				FromSrcAttrIDMap[(int32)SrcAttrTri[2]]);

			///--  We have to do some clean-up on the shared Attrs because the MeshIn format might support wild stuff.. --///
			{
				// MeshIn may attach multiple mesh vertices to the same attribute element.  DynamicMesh does not.
				// if we have already used this element for a different mesh vertex, split it.
				const FIndex3i ParentTriangle = MeshOut.GetTriangle(TriID);
				for (int i = 0; i < 3; ++i)
				{
					int32 ParentVID = Overlay->GetParentVertex(AttrTri[i]);
					if (ParentVID != FDynamicMesh3::InvalidID && ParentVID != ParentTriangle[i])
					{
						const AttrType AttrValue = SharedAttrValues(SrcAttrTri[i]);
						AttrTri[i] = Overlay->AppendElement(AttrValue);
					}
				}

				// MeshIn may have degenerate attr tris.  Dynamic Mesh does not.
				// if the attr tri is degenerate we split the degenerate attr edge by adding two new attrs
				// in its place, or if it is totally degenerate we add 3 new attrs

				if (AttrTri[0] == AttrTri[1] && AttrTri[0] == AttrTri[2])
				{
					const AttrType AttrValue = SharedAttrValues(SrcAttrTri[0]);
					AttrTri[0] = Overlay->AppendElement(AttrValue);
					AttrTri[1] = Overlay->AppendElement(AttrValue);
					AttrTri[2] = Overlay->AppendElement(AttrValue);

				}
				else
				{
					if (AttrTri[0] == AttrTri[1])
					{
						const AttrType AttrValue = SharedAttrValues(SrcAttrTri[0]);
						AttrTri[0] = Overlay->AppendElement(AttrValue);
						AttrTri[1] = Overlay->AppendElement(AttrValue);
					}
					if (AttrTri[0] == AttrTri[2])
					{
						const AttrType AttrValue = SharedAttrValues(SrcAttrTri[0]);
						AttrTri[0] = Overlay->AppendElement(AttrValue);
						AttrTri[2] = Overlay->AppendElement(AttrValue);
					}
					if (AttrTri[1] == AttrTri[2])
					{
						const AttrType AttrValue = SharedAttrValues(SrcAttrTri[1]);
						AttrTri[1] = Overlay->AppendElement(AttrValue);
						AttrTri[2] = Overlay->AppendElement(AttrValue);
					}
				}
			}

			// set the triangle in the overlay
			Overlay->SetTriangle(TriID, AttrTri);

		}
	}

	template <typename AttrType, typename SrcAttrIDType>
	void PopulateOverlay(typename TOverlayTraits<AttrType>::OverlayType* Overlay,
		const TArray<SrcAttrIDType>& SrcAttrIDs,
		TFunctionRef<AttrType(const SrcAttrIDType& AttrID)>  SharedAttrValues,
		TFunctionRef<bool(const SrcTriIDType& SrcTriID, SrcAttrIDType& AttrID0, SrcAttrIDType& AttrID1, SrcAttrIDType& AttrID2)> AttrTriIndices,
		TFunctionRef<AttrType(const SrcWedgeIDType& WegdeID)> WedgeAttrs,
		TFunctionRef<void(const SrcTriIDType& SrcTriID, SrcWedgeIDType& WID0, SrcWedgeIDType& WID1, SrcWedgeIDType& WID2)> TriToWedges)
	{
		PopulateOverlayFromSharedAttr(Overlay, SrcAttrIDs, SharedAttrValues, AttrTriIndices);
		// Populate the overlay tris that were left empty by the shared attrs
		PopulateOverlayFromWedgeAttr(Overlay, WedgeAttrs, TriToWedges);
	}
};

} } // end namespace UE::Geometry